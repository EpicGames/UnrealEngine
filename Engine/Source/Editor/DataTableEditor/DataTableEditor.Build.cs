// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class DataTableEditor : ModuleRules
{
	public DataTableEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("LevelEditor");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"ApplicationCore",
				"Engine",
                "InputCore",
				"Slate",
				"SlateCore",
                "PropertyEditor",
				"EditorFramework",
				"UnrealEd",
				"Json",
				"DeveloperSettings",
				"ToolMenus"
			}
			);

		DynamicallyLoadedModuleNames.Add("WorkspaceMenuStructure");
	}
}
