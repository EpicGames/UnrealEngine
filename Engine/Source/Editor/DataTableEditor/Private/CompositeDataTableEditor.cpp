// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositeDataTableEditor.h"

#include "DataTableEditorModule.h"
#include "Delegates/Delegate.h"
#include "DetailsViewArgs.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/CompositeDataTable.h"
#include "Engine/DataTable.h"
#include "Framework/Docking/TabManager.h"
#include "IDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SCompositeRowEditor.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBorder.h"

class SRowEditor;
class SWidget;
 
#define LOCTEXT_NAMESPACE "CompositeDataTableEditor"

const FName FCompositeDataTableEditor::PropertiesTabId("CompositeDataTableEditor_Properties");
const FName FCompositeDataTableEditor::StackTabId("CompositeDataTableEditor_Stack");


FCompositeDataTableEditor::FCompositeDataTableEditor()
{
}

FCompositeDataTableEditor::~FCompositeDataTableEditor()
{
}

void FCompositeDataTableEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FDataTableEditor::RegisterTabSpawners(InTabManager);

	CreateAndRegisterPropertiesTab(InTabManager);
}

void FCompositeDataTableEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FDataTableEditor::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(PropertiesTabId);
	InTabManager->UnregisterTabSpawner(StackTabId);

	DetailsView.Reset();
	StackTabWidget.Reset();
}

void FCompositeDataTableEditor::CreateAndRegisterRowEditorTab(const TSharedRef<class FTabManager>& InTabManager)
{
	// no row editor in the composite data tables
	RowEditorTabWidget.Reset();
}

void FCompositeDataTableEditor::CreateAndRegisterPropertiesTab(const TSharedRef<class FTabManager>& InTabManager)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	InTabManager->RegisterTabSpawner(PropertiesTabId, FOnSpawnTab::CreateSP(this, &FCompositeDataTableEditor::SpawnTab_Properties))
		.SetDisplayName(LOCTEXT("PropertiesTab", "Properties"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());
}

void FCompositeDataTableEditor::InitDataTableEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UDataTable* Table)
{
	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_CompositeDataTableEditor_temp_Layout_v2")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.3f)	
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.3f)
					->SetHideTabWell(true)
					->AddTab(PropertiesTabId, ETabState::OpenedTab)
				)
// 				->Split
// 				(
// 					FTabManager::NewStack()		
// 					->SetHideTabWell(true)
// 					->AddTab(StackTabId, ETabState::OpenedTab)
// 				)

			)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->AddTab(DataTableTabId, ETabState::OpenedTab)
				)
// 				->Split
// 				(
// 					FTabManager::NewStack()
// 					->AddTab(RowEditorTabId, ETabState::OpenedTab)
// 				)
			)
		);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, FDataTableEditorModule::DataTableEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, Table);

	FDataTableEditorModule& DataTableEditorModule = FModuleManager::LoadModuleChecked<FDataTableEditorModule>("DataTableEditor");
	AddMenuExtender(DataTableEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	// Support undo/redo
	GEditor->RegisterForUndo(this);

	if (DetailsView.IsValid())
	{
		// Make sure details window is pointing to our object
		DetailsView->SetObject(GetEditingObject());
	}
}

bool FCompositeDataTableEditor::CanEditRows() const
{
	return false;
}

TSharedRef<SDockTab> FCompositeDataTableEditor::SpawnTab_Stack(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == StackTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("StackTitle", "Datatable Stack"))
		.TabColorScale(GetTabColorScale())
		[
			SNew(SBorder)
			.Padding(2)
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Fill)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				StackTabWidget.ToSharedRef()
			]
		];
}

TSharedRef<SDockTab> FCompositeDataTableEditor::SpawnTab_Properties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == PropertiesTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("PropertiesTitle", "Properties"))
		.TabColorScale(GetTabColorScale())
		[
			DetailsView.ToSharedRef()
		];
}

TSharedRef<SWidget> FCompositeDataTableEditor::CreateStackBox()
{
	UDataTable* Table = Cast<UDataTable>(GetEditingObject());

	// Support undo/redo
	if (Table)
	{
		Table->SetFlags(RF_Transactional);
	}

	return CreateRowEditor(Table);
}

TSharedRef<SRowEditor> FCompositeDataTableEditor::CreateRowEditor(UDataTable* Table)
{
	UCompositeDataTable* DataTable = Cast<UCompositeDataTable>(Table);
	return SNew(SCompositeRowEditor, DataTable);

}

#undef LOCTEXT_NAMESPACE
