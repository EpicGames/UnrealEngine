// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataTableEditor.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/IToolkit.h"

class FName;
class FSpawnTabArgs;
class SDockTab;
class SRowEditor;
class SWidget;
class UDataTable;

/** Viewer/editor for a CompositeDataTable */
class FCompositeDataTableEditor : public FDataTableEditor
{
public:

	/** Constructor */
	FCompositeDataTableEditor();

	/** Destructor */
	virtual ~FCompositeDataTableEditor();

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	virtual void InitDataTableEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UDataTable* Table) override;
	virtual bool CanEditRows() const override;

	/**	Spawns the tab with the details view inside */
	TSharedRef<SDockTab> SpawnTab_Properties(const FSpawnTabArgs& Args);

	/**	Spawns the tab with the stack of data tables inside */
	TSharedRef<SDockTab> SpawnTab_Stack(const FSpawnTabArgs& Args);

protected:

	virtual void CreateAndRegisterRowEditorTab(const TSharedRef<class FTabManager>& InTabManager) override;

	/** Helper function for creating and registering the tab containing the properties tab */
	virtual void CreateAndRegisterPropertiesTab(const TSharedRef<class FTabManager>& InTabManager);

private:
	TSharedRef<SWidget> CreateStackBox();
	
	virtual TSharedRef<SRowEditor> CreateRowEditor(UDataTable* Table);

	/** Details view */
	TSharedPtr< class IDetailsView > DetailsView;

	/** UI for the "Stack" tab */
	TSharedPtr<SWidget> StackTabWidget;

	static const FName PropertiesTabId;
	static const FName StackTabId;
};
