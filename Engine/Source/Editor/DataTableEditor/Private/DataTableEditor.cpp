// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTableEditor.h"

#include "AssetRegistry/AssetData.h"
#include "Containers/Map.h"
#include "CoreGlobals.h"
#include "DataTableEditorModule.h"
#include "DataTableEditorSettings.h"
#include "DataTableUtils.h"
#include "DetailsViewArgs.h"
#include "Dom/JsonObject.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/DataTable.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Layout/Overscroll.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Views/ITypedTableView.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IDetailsView.h"
#include "IDocumentation.h"
#include "Internationalization/Internationalization.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/ColorList.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/FeedbackContext.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Policies/PrettyJsonPrintPolicy.h"
#include "PropertyEditorModule.h"
#include "Rendering/SlateRenderer.h"
#include "SDataTableListViewRow.h"
#include "SRowEditor.h"
#include "ScopedTransaction.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "SheetSerialization.h"
#include "SlotBase.h"
#include "SourceCodeNavigation.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/TypeHash.h"
#include "Textures/SlateIcon.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/OverriddenPropertySet.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectBaseUtility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

class ITableRow;
class STableViewBase;
class SWidget;

#define LOCTEXT_NAMESPACE "DataTableEditor"

const FName FDataTableEditor::DataTableTabId("DataTableEditor_DataTable");
const FName FDataTableEditor::DataTableDetailsTabId("DataTableEditor_DataTableDetails");
const FName FDataTableEditor::RowEditorTabId("DataTableEditor_RowEditor");
const FName FDataTableEditor::RowNameColumnId("RowName");
const FName FDataTableEditor::RowNumberColumnId("RowNumber");
const FName FDataTableEditor::RowDragDropColumnId("RowDragDrop");

class SDataTableModeSeparator : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SDataTableModeSeparator) {}
	SLATE_END_ARGS()

		void Construct(const FArguments& InArg)
	{
		SBorder::Construct(
			SBorder::FArguments()
			.BorderImage(FAppStyle::GetBrush("BlueprintEditor.PipelineSeparator"))
			.Padding(0.0f)
		);
	}

	// SWidget interface
	virtual FVector2D ComputeDesiredSize(float) const override
	{
		const float Height = 20.0f;
		const float Thickness = 16.0f;
		return FVector2D(Thickness, Height);
	}
	// End of SWidget interface
};


void FDataTableEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_Data Table Editor", "Data Table Editor"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	CreateAndRegisterDataTableTab(InTabManager);
	CreateAndRegisterDataTableDetailsTab(InTabManager);
	CreateAndRegisterRowEditorTab(InTabManager);
}

void FDataTableEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(DataTableTabId);
	InTabManager->UnregisterTabSpawner(DataTableDetailsTabId);
	InTabManager->UnregisterTabSpawner(RowEditorTabId);

	DataTableTabWidget.Reset();
	RowEditorTabWidget.Reset();
}

void FDataTableEditor::CreateAndRegisterDataTableTab(const TSharedRef<class FTabManager>& InTabManager)
{
	DataTableTabWidget = CreateContentBox();

	InTabManager->RegisterTabSpawner(DataTableTabId, FOnSpawnTab::CreateSP(this, &FDataTableEditor::SpawnTab_DataTable))
		.SetDisplayName(LOCTEXT("DataTableTab", "Data Table"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());
}

void FDataTableEditor::CreateAndRegisterDataTableDetailsTab(const TSharedRef<class FTabManager>& InTabManager)
{
	FPropertyEditorModule & EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	PropertyView = EditModule.CreateDetailView(DetailsViewArgs);

	InTabManager->RegisterTabSpawner(DataTableDetailsTabId, FOnSpawnTab::CreateSP(this, &FDataTableEditor::SpawnTab_DataTableDetails))
		.SetDisplayName(LOCTEXT("DataTableDetailsTab", "Data Table Details"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());
}

void FDataTableEditor::CreateAndRegisterRowEditorTab(const TSharedRef<class FTabManager>& InTabManager)
{
	RowEditorTabWidget = CreateRowEditorBox();

	InTabManager->RegisterTabSpawner(RowEditorTabId, FOnSpawnTab::CreateSP(this, &FDataTableEditor::SpawnTab_RowEditor))
		.SetDisplayName(LOCTEXT("RowEditorTab", "Row Editor"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef());
}

FDataTableEditor::FDataTableEditor()
	: RowNameColumnWidth(0)
	, RowNumberColumnWidth(0)
	, HighlightedVisibleRowIndex(INDEX_NONE)
	, SortMode(EColumnSortMode::Ascending)
{
}

FDataTableEditor::~FDataTableEditor()
{
	GEditor->UnregisterForUndo(this);

	UDataTable* Table = GetEditableDataTable();
	if (Table)
	{
		SaveLayoutData();
	}
}

void FDataTableEditor::PostUndo(bool bSuccess)
{
	HandleUndoRedo();
}

void FDataTableEditor::PostRedo(bool bSuccess)
{
	HandleUndoRedo();
}

void FDataTableEditor::HandleUndoRedo()
{
	const UDataTable* Table = GetDataTable();
	if (Table)
	{
		HandlePostChange();
		CallbackOnDataTableUndoRedo.ExecuteIfBound();
	}
}

void FDataTableEditor::PreChange(const class UUserDefinedStruct* Struct, FStructureEditorUtils::EStructureEditorChangeInfo Info)
{
}

void FDataTableEditor::PostChange(const class UUserDefinedStruct* Struct, FStructureEditorUtils::EStructureEditorChangeInfo Info)
{
	const UDataTable* Table = GetDataTable();
	if (Struct && Table && (Table->GetRowStruct() == Struct))
	{
		HandlePostChange();
	}
}

void FDataTableEditor::SelectionChange(const UDataTable* Changed, FName RowName)
{
	const UDataTable* Table = GetDataTable();
	if (Changed == Table)
	{
		const bool bSelectionChanged = HighlightedRowName != RowName;
		SetHighlightedRow(RowName);

		if (bSelectionChanged)
		{
			CallbackOnRowHighlighted.ExecuteIfBound(HighlightedRowName);
		}
	}
}

void FDataTableEditor::PreChange(const UDataTable* Changed, FDataTableEditorUtils::EDataTableChangeInfo Info)
{
}

void FDataTableEditor::PostChange(const UDataTable* Changed, FDataTableEditorUtils::EDataTableChangeInfo Info)
{
	UDataTable* Table = GetEditableDataTable();
	if (Changed == Table)
	{
		// Don't need to notify the DataTable about changes, that's handled before this
		HandlePostChange();
	}
}

const UDataTable* FDataTableEditor::GetDataTable() const
{
	return Cast<const UDataTable>(GetEditingObject());
}

void FDataTableEditor::HandlePostChange()
{
	// We need to cache and restore the selection here as RefreshCachedDataTable will re-create the list view items
	const FName CachedSelection = HighlightedRowName;
	HighlightedRowName = NAME_None;
	RefreshCachedDataTable(CachedSelection, true/*bUpdateEvenIfValid*/);
}

void FDataTableEditor::InitDataTableEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UDataTable* Table )
{
	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout( "Standalone_DataTableEditor_Layout_v6" )
	->AddArea
	(
		FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewStack()
			->AddTab(DataTableTabId, ETabState::OpenedTab)
			->AddTab(DataTableDetailsTabId, ETabState::OpenedTab)
			->SetForegroundTab(DataTableTabId)
		)
		->Split
		(
			FTabManager::NewStack()
			->AddTab(RowEditorTabId, ETabState::OpenedTab)
		)
	);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor( Mode, InitToolkitHost, FDataTableEditorModule::DataTableEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, Table );

	FDataTableEditorModule& DataTableEditorModule = FModuleManager::LoadModuleChecked<FDataTableEditorModule>( "DataTableEditor" );
	AddMenuExtender(DataTableEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
	
	TSharedPtr<FExtender> ToolbarExtender = DataTableEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects());
	ExtendToolbar(ToolbarExtender);
	
	AddToolbarExtender(ToolbarExtender);

	RegenerateMenusAndToolbars();

	// Support undo/redo
	GEditor->RegisterForUndo(this);

	// @todo toolkit world centric editing
	/*// Setup our tool's layout
	if( IsWorldCentricAssetEditor() )
	{
		const FString TabInitializationPayload(TEXT(""));		// NOTE: Payload not currently used for table properties
		SpawnToolkitTab( DataTableTabId, TabInitializationPayload, EToolkitTabSpot::Details );
	}*/

	// asset editor commands here
	ToolkitCommands->MapAction(FGenericCommands::Get().Copy, FExecuteAction::CreateSP(this, &FDataTableEditor::CopySelectedRow), FCanExecuteAction::CreateSP(this, &FDataTableEditor::CanEditTable));
	ToolkitCommands->MapAction(FGenericCommands::Get().Paste, FExecuteAction::CreateSP(this, &FDataTableEditor::PasteOnSelectedRow), FCanExecuteAction::CreateSP(this, &FDataTableEditor::CanEditTable));
	ToolkitCommands->MapAction(FGenericCommands::Get().Duplicate, FExecuteAction::CreateSP(this, &FDataTableEditor::DuplicateSelectedRow), FCanExecuteAction::CreateSP(this, &FDataTableEditor::CanEditTable));
	ToolkitCommands->MapAction(FGenericCommands::Get().Rename, FExecuteAction::CreateSP(this, &FDataTableEditor::RenameSelectedRowCommand), FCanExecuteAction::CreateSP(this, &FDataTableEditor::CanEditTable));
	ToolkitCommands->MapAction(FGenericCommands::Get().Delete, FExecuteAction::CreateSP(this, &FDataTableEditor::DeleteSelectedRow), FCanExecuteAction::CreateSP(this, &FDataTableEditor::CanEditTable));
}

bool FDataTableEditor::CanEditRows() const
{
	return true;
}

FName FDataTableEditor::GetToolkitFName() const
{
	return FName("DataTableEditor");
}

FString FDataTableEditor::GetDocumentationLink() const
{
	return FString(TEXT("Gameplay/DataDriven"));
}

void FDataTableEditor::OnAddClicked()
{
	UDataTable* Table = GetEditableDataTable();

	if (Table)
	{		
		FName NewName = DataTableUtils::MakeValidName(TEXT("NewRow"));
		while (Table->GetRowMap().Contains(NewName))
		{
			NewName.SetNumber(NewName.GetNumber() + 1);
		}

		FDataTableEditorUtils::AddRow(Table, NewName);
		FDataTableEditorUtils::SelectRow(Table, NewName);

		SetDefaultSort();
	}
}

void FDataTableEditor::OnRemoveClicked()
{
	DeleteSelectedRow();
}

FReply FDataTableEditor::OnMoveRowClicked(FDataTableEditorUtils::ERowMoveDirection MoveDirection)
{
	UDataTable* Table = GetEditableDataTable();

	if (Table)
	{
		FDataTableEditorUtils::MoveRow(Table, HighlightedRowName, MoveDirection);
	}
	return FReply::Handled();
}

FReply FDataTableEditor::OnMoveToExtentClicked(FDataTableEditorUtils::ERowMoveDirection MoveDirection)
{
	UDataTable* Table = GetEditableDataTable();

	if (Table)
	{
		// We move by the row map size, as FDataTableEditorUtils::MoveRow will automatically clamp this as appropriate
		FDataTableEditorUtils::MoveRow(Table, HighlightedRowName, MoveDirection, Table->GetRowMap().Num());
		FDataTableEditorUtils::SelectRow(Table, HighlightedRowName);

		SetDefaultSort();
	}
	return FReply::Handled();
}

void FDataTableEditor::OnCopyClicked()
{
	UDataTable* Table = GetEditableDataTable();
	if (Table)
	{
		CopySelectedRow();
	}
}

void FDataTableEditor::OnPasteClicked()
{
	UDataTable* Table = GetEditableDataTable();
	if (Table)
	{
		PasteOnSelectedRow();
	}
}

void FDataTableEditor::OnDuplicateClicked()
{
	UDataTable* Table = GetEditableDataTable();
	if (Table)
	{
		DuplicateSelectedRow();
	}
}

void FDataTableEditor::OnCopySheetClicked()
{
	UDataTable* TablePtr = Cast<UDataTable>(GetEditingObject());
	if (!TablePtr)
	{
		return;
	}

	TArray<FDataTableRowData> Rows;
	for (const TTuple<FName, unsigned char*>& RowMap : TablePtr->GetRowMap())
	{
		Rows.Emplace(RowMap.Key, RowMap.Value);
	}

	const FString ClipboardValue = USheetSerializationSubSystem::Get()->ExportForSheet(Rows, TablePtr, true);
	FPlatformApplicationMisc::ClipboardCopy(*ClipboardValue);
}

void FDataTableEditor::OnPasteSheetClicked()
{
	UDataTable* TablePtr = Cast<UDataTable>(GetEditingObject());
	if (!TablePtr)
	{
		return;
	}

	FString ClipboardValue;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardValue);
	ClipboardValue.TrimEndInline();

	const FScopedTransaction Transaction(LOCTEXT("PasteSheetOverTable", "Paste Spreadsheet Over Data Table"));
	TablePtr->Modify();

	FDataTableEditorUtils::BroadcastPreChange(TablePtr, FDataTableEditorUtils::EDataTableChangeInfo::RowList);

	if (!USheetSerializationSubSystem::Get()->ImportFromSheet(ClipboardValue, TablePtr, GWarn))
	{
		FNotificationInfo Info(LOCTEXT("FailedPasteSheet", "Failed to paste rows from sheet"));
		FSlateNotificationManager::Get().AddNotification(Info);
	}

	TablePtr->HandleDataTableChanged(NAME_None);
	TablePtr->MarkPackageDirty();
	FDataTableEditorUtils::BroadcastPostChange(TablePtr, FDataTableEditorUtils::EDataTableChangeInfo::RowList);
}

bool FDataTableEditor::CanEditTable() const
{
	return HighlightedRowName != NAME_None;
}

void FDataTableEditor::SetDefaultSort()
{
	SortMode = EColumnSortMode::Ascending;
	SortByColumn = FDataTableEditor::RowNumberColumnId;
}

EColumnSortMode::Type FDataTableEditor::GetColumnSortMode(const FName ColumnId) const
{
	if (SortByColumn != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}

void FDataTableEditor::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	int32 ColumnIndex;

	SortMode = InSortMode;
	SortByColumn = ColumnId;

	for (ColumnIndex = 0; ColumnIndex < AvailableColumns.Num(); ++ColumnIndex)
	{
		if (AvailableColumns[ColumnIndex]->ColumnId == ColumnId)
		{
			break;
		}
	}

	if (AvailableColumns.IsValidIndex(ColumnIndex))
	{
		if (InSortMode == EColumnSortMode::Ascending)
		{
			VisibleRows.Sort([ColumnIndex](const FDataTableEditorRowListViewDataPtr& first, const FDataTableEditorRowListViewDataPtr& second)
			{					
				int32 Result = (first->CellData[ColumnIndex].ToString()).Compare(second->CellData[ColumnIndex].ToString());

				if (!Result)
				{
					return first->RowNum < second->RowNum;

				}

				return Result < 0;
			});

		}
		else if (InSortMode == EColumnSortMode::Descending)
		{
			VisibleRows.Sort([ColumnIndex](const FDataTableEditorRowListViewDataPtr& first, const FDataTableEditorRowListViewDataPtr& second)
			{
				int32 Result = (first->CellData[ColumnIndex].ToString()).Compare(second->CellData[ColumnIndex].ToString());

				if (!Result)
				{
					return first->RowNum > second->RowNum;
				}

				return Result > 0;
			});
		}
	}

	CellsListView->RequestListRefresh();
}

void FDataTableEditor::OnColumnNumberSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortMode = InSortMode;
	SortByColumn = ColumnId;

	if (InSortMode == EColumnSortMode::Ascending)
	{
		VisibleRows.Sort([](const FDataTableEditorRowListViewDataPtr& first, const FDataTableEditorRowListViewDataPtr& second)
		{
			return first->RowNum < second->RowNum;
		});
	}
	else if (InSortMode == EColumnSortMode::Descending)
	{
		VisibleRows.Sort([](const FDataTableEditorRowListViewDataPtr& first, const FDataTableEditorRowListViewDataPtr& second)
		{
			return first->RowNum > second->RowNum;
		});
	}

	CellsListView->RequestListRefresh();
}

void FDataTableEditor::OnColumnNameSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortMode = InSortMode;
	SortByColumn = ColumnId;

	if (InSortMode == EColumnSortMode::Ascending)
	{
		VisibleRows.Sort([](const FDataTableEditorRowListViewDataPtr& first, const FDataTableEditorRowListViewDataPtr& second)
		{
			return (first->DisplayName).ToString() < (second->DisplayName).ToString();
		});
	}
	else if (InSortMode == EColumnSortMode::Descending)
	{
		VisibleRows.Sort([](const FDataTableEditorRowListViewDataPtr& first, const FDataTableEditorRowListViewDataPtr& second)
		{
			return (first->DisplayName).ToString() > (second->DisplayName).ToString();
		});
	}

	CellsListView->RequestListRefresh();
}

void FDataTableEditor::OnEditDataTableStructClicked()
{
	const UDataTable* DataTable = GetDataTable();
	if (DataTable)
	{
		const UScriptStruct* ScriptStruct = DataTable->GetRowStruct();

		if (ScriptStruct)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ScriptStruct->GetPathName());
			if (FSourceCodeNavigation::CanNavigateToStruct(ScriptStruct))
			{
				FSourceCodeNavigation::NavigateToStruct(ScriptStruct);
			}
		}
	}
}

void FDataTableEditor::ExtendToolbar(TSharedPtr<FExtender> Extender)
{
	Extender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateSP(this, &FDataTableEditor::FillToolbar)
	);

}

void FDataTableEditor::FillToolbar(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.BeginSection("DataTableCommands");
	{
		ToolbarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateSP(this, &FDataTableEditor::Reimport_Execute),
				FCanExecuteAction::CreateSP(this, &FDataTableEditor::CanReimport)),
			NAME_None,
			LOCTEXT("ReimportText", "Reimport"),
			LOCTEXT("ReimportTooltip", "Reimport this DataTable"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Reimport"));

		ToolbarBuilder.AddSeparator();

		ToolbarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &FDataTableEditor::OnAddClicked)),
			NAME_None,
			LOCTEXT("AddIconText", "Add"),
			LOCTEXT("AddRowToolTip", "Add a new row to the Data Table"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus"));
		ToolbarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateSP(this, &FDataTableEditor::OnCopyClicked),
				FCanExecuteAction::CreateSP(this, &FDataTableEditor::CanEditTable)),
			NAME_None,
			LOCTEXT("CopyIconText", "Copy"),
			LOCTEXT("CopyToolTip", "Copy the currently selected row"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"));
		ToolbarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateSP(this, &FDataTableEditor::OnPasteClicked),
				FCanExecuteAction::CreateSP(this, &FDataTableEditor::CanEditTable)),
			NAME_None,
			LOCTEXT("PasteIconText", "Paste"),
			LOCTEXT("PasteToolTip", "Paste on the currently selected row"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Paste"));
		ToolbarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateSP(this, &FDataTableEditor::OnDuplicateClicked),
				FCanExecuteAction::CreateSP(this, &FDataTableEditor::CanEditTable)),
			NAME_None,
			LOCTEXT("DuplicateIconText", "Duplicate"),
			LOCTEXT("DuplicateToolTip", "Duplicate the currently selected row"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Duplicate"));
		ToolbarBuilder.AddToolBarButton(
			FUIAction(
				FExecuteAction::CreateSP(this, &FDataTableEditor::OnRemoveClicked),
				FCanExecuteAction::CreateSP(this, &FDataTableEditor::CanEditTable)),
			NAME_None,
			LOCTEXT("RemoveRowIconText", "Remove"),
			LOCTEXT("RemoveRowToolTip", "Remove the currently selected row from the Data Table"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"));

		ToolbarBuilder.AddSeparator();

		ToolbarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &FDataTableEditor::OnCopySheetClicked)),
			NAME_None,
			LOCTEXT("CopySheetText", "Copy Sheet"),
			LOCTEXT("CopySheetTooltip", "Copy this DataTable as a spreadsheet"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"));
		ToolbarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateSP(this, &FDataTableEditor::OnPasteSheetClicked)),
			NAME_None,
			LOCTEXT("PasteSheetText", "Paste Sheet"),
			LOCTEXT("PasteSheetTooltip", "Paste spreadsheet values over the DataTable"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Paste"));
	}
	ToolbarBuilder.EndSection();

}

UDataTable* FDataTableEditor::GetEditableDataTable() const
{
	return Cast<UDataTable>(GetEditingObject());
}

FText FDataTableEditor::GetBaseToolkitName() const
{
	return LOCTEXT( "AppLabel", "DataTable Editor" );
}

FString FDataTableEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "DataTable ").ToString();
}

FLinearColor FDataTableEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.0f, 0.0f, 0.2f, 0.5f );
}

FSlateColor FDataTableEditor::GetRowTextColor(FName RowName) const
{
	if (RowName == HighlightedRowName)
	{
		return FSlateColor(FColorList::Orange);
	}
	return FSlateColor::UseForeground();
}

FText FDataTableEditor::GetCellText(FDataTableEditorRowListViewDataPtr InRowDataPointer, int32 ColumnIndex) const
{
	if (InRowDataPointer.IsValid() && ColumnIndex < InRowDataPointer->CellData.Num())
	{
		return InRowDataPointer->CellData[ColumnIndex];
	}

	return FText();
}

FText FDataTableEditor::GetCellToolTipText(FDataTableEditorRowListViewDataPtr InRowDataPointer, int32 ColumnIndex) const
{
	FText TooltipText;

	if (ColumnIndex < AvailableColumns.Num())
	{
		TooltipText = AvailableColumns[ColumnIndex]->DisplayName;
	}

	if (InRowDataPointer.IsValid() && ColumnIndex < InRowDataPointer->CellData.Num())
	{
		TooltipText = FText::Format(LOCTEXT("ColumnRowNameFmt", "{0}: {1}"), TooltipText, InRowDataPointer->CellData[ColumnIndex]);
	}

	return TooltipText;
}

float FDataTableEditor::GetRowNumberColumnWidth() const
{
	return RowNumberColumnWidth;
}

void FDataTableEditor::RefreshRowNumberColumnWidth()
{

	TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const FTextBlockStyle& CellTextStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("DataTableEditor.CellText");
	const float CellPadding = 10.0f;

	for (const FDataTableEditorRowListViewDataPtr& RowData : AvailableRows)
	{
		const float RowNumberWidth = (float)FontMeasure->Measure(FString::FromInt(RowData->RowNum), CellTextStyle.Font).X + CellPadding;
		RowNumberColumnWidth = FMath::Max(RowNumberColumnWidth, RowNumberWidth);
	}

}

void FDataTableEditor::OnRowNumberColumnResized(const float NewWidth)
{
	RowNumberColumnWidth = NewWidth;
}

float FDataTableEditor::GetRowNameColumnWidth() const
{
	return RowNameColumnWidth;
}

void FDataTableEditor::RefreshRowNameColumnWidth()
{
	
	TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const FTextBlockStyle& CellTextStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>("DataTableEditor.CellText");
	static const float CellPadding = 10.0f;

	for (const FDataTableEditorRowListViewDataPtr& RowData : AvailableRows)
	{
		const float RowNameWidth = (float)FontMeasure->Measure(RowData->DisplayName, CellTextStyle.Font).X + CellPadding;
		RowNameColumnWidth = FMath::Max(RowNameColumnWidth, RowNameWidth);
	}
	
}

float FDataTableEditor::GetColumnWidth(const int32 ColumnIndex) const
{
	if (ColumnWidths.IsValidIndex(ColumnIndex))
	{
		return ColumnWidths[ColumnIndex].CurrentWidth;
	}
	return 0.0f;
}

void FDataTableEditor::OnColumnResized(const float NewWidth, const int32 ColumnIndex)
{
	if (ColumnWidths.IsValidIndex(ColumnIndex))
	{
		FColumnWidth& ColumnWidth = ColumnWidths[ColumnIndex];
		ColumnWidth.bIsAutoSized = false;
		ColumnWidth.CurrentWidth = NewWidth;

		// Update the persistent column widths in the layout data
		{
			if (!LayoutData.IsValid())
			{
				LayoutData = MakeShareable(new FJsonObject());
			}

			TSharedPtr<FJsonObject> LayoutColumnWidths;
			if (!LayoutData->HasField(TEXT("ColumnWidths")))
			{
				LayoutColumnWidths = MakeShareable(new FJsonObject());
				LayoutData->SetObjectField(TEXT("ColumnWidths"), LayoutColumnWidths);
			}
			else
			{
				LayoutColumnWidths = LayoutData->GetObjectField(TEXT("ColumnWidths"));
			}

			const FString& ColumnName = AvailableColumns[ColumnIndex]->ColumnId.ToString();
			LayoutColumnWidths->SetNumberField(ColumnName, NewWidth);
		}
	}
}

void FDataTableEditor::OnRowNameColumnResized(const float NewWidth)
{
	RowNameColumnWidth = NewWidth;
}

void FDataTableEditor::LoadLayoutData()
{
	LayoutData.Reset();

	const UDataTable* Table = GetDataTable();
	if (!Table)
	{
		return;
	}

	const FString LayoutDataFilename = FPaths::ProjectSavedDir() / TEXT("AssetData") / TEXT("DataTableEditorLayout") / Table->GetName() + TEXT(".json");

	FString JsonText;
	if (FFileHelper::LoadFileToString(JsonText, *LayoutDataFilename))
	{
		TSharedRef< TJsonReader<TCHAR> > JsonReader = TJsonReaderFactory<TCHAR>::Create(JsonText);
		FJsonSerializer::Deserialize(JsonReader, LayoutData);
	}
}

void FDataTableEditor::SaveLayoutData()
{
	const UDataTable* Table = GetDataTable();
	if (!Table || !LayoutData.IsValid())
	{
		return;
	}

	const FString LayoutDataFilename = FPaths::ProjectSavedDir() / TEXT("AssetData") / TEXT("DataTableEditorLayout") / Table->GetName() + TEXT(".json");

	FString JsonText;
	TSharedRef< TJsonWriter< TCHAR, TPrettyJsonPrintPolicy<TCHAR> > > JsonWriter = TJsonWriterFactory< TCHAR, TPrettyJsonPrintPolicy<TCHAR> >::Create(&JsonText);
	if (FJsonSerializer::Serialize(LayoutData.ToSharedRef(), JsonWriter))
	{
		FFileHelper::SaveStringToFile(JsonText, *LayoutDataFilename);
	}
}

TSharedRef<ITableRow> FDataTableEditor::MakeRowWidget(FDataTableEditorRowListViewDataPtr InRowDataPtr, const TSharedRef<STableViewBase>& OwnerTable)
{
	return
		SNew(SDataTableListViewRow, OwnerTable)
		.DataTableEditor(SharedThis(this))
		.RowDataPtr(InRowDataPtr)
		.IsEditable(CanEditRows());
}

TSharedRef<SWidget> FDataTableEditor::MakeCellWidget(FDataTableEditorRowListViewDataPtr InRowDataPtr, const int32 InRowIndex, const FName& InColumnId)
{
	int32 ColumnIndex = 0;
	for (; ColumnIndex < AvailableColumns.Num(); ++ColumnIndex)
	{
		const FDataTableEditorColumnHeaderDataPtr& ColumnData = AvailableColumns[ColumnIndex];
		if (ColumnData->ColumnId == InColumnId)
		{
			break;
		}
	}

	// Valid column ID?
	if (AvailableColumns.IsValidIndex(ColumnIndex) && InRowDataPtr->CellData.IsValidIndex(ColumnIndex))
	{
		return SNew(SBox)
			.Padding(FMargin(4, 2, 4, 2))
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "DataTableEditor.CellText")
				.ColorAndOpacity(this, &FDataTableEditor::GetRowTextColor, InRowDataPtr->RowId)
				.Text(this, &FDataTableEditor::GetCellText, InRowDataPtr, ColumnIndex)
				.HighlightText(this, &FDataTableEditor::GetFilterText)
				.ToolTipText(this, &FDataTableEditor::GetCellToolTipText, InRowDataPtr, ColumnIndex)
			];
	}

	return SNullWidget::NullWidget;
}

void FDataTableEditor::OnRowSelectionChanged(FDataTableEditorRowListViewDataPtr InNewSelection, ESelectInfo::Type InSelectInfo)
{
	const bool bSelectionChanged = !InNewSelection.IsValid() || InNewSelection->RowId != HighlightedRowName;
	const FName NewRowName = (InNewSelection.IsValid()) ? InNewSelection->RowId : NAME_None;

	SetHighlightedRow(NewRowName);

	if (bSelectionChanged)
	{
		CallbackOnRowHighlighted.ExecuteIfBound(HighlightedRowName);
	}
}

void FDataTableEditor::CopySelectedRow()
{
	UDataTable* TablePtr = Cast<UDataTable>(GetEditingObject());
	uint8* RowPtr = TablePtr ? TablePtr->GetRowMap().FindRef(HighlightedRowName) : nullptr;

	if (!RowPtr || !TablePtr->RowStruct)
		return;

	FString ClipboardValue;
	if (GetDefault<UDataTableEditorSettings>()->bCopyAsSpreadsheetCells)
	{
		TArray<FDataTableRowData> Rows;
		Rows.Emplace(HighlightedRowName, RowPtr);
		ClipboardValue = USheetSerializationSubSystem::Get()->ExportForSheet(Rows, TablePtr, false);
	}
	else
	{
		TablePtr->RowStruct->ExportText(ClipboardValue, RowPtr, RowPtr, TablePtr, PPF_Copy, nullptr);
	}

	FPlatformApplicationMisc::ClipboardCopy(*ClipboardValue);
}

void FDataTableEditor::PasteOnSelectedRow()
{
	UDataTable* TablePtr = Cast<UDataTable>(GetEditingObject());
	uint8* RowPtr = TablePtr ? TablePtr->GetRowMap().FindRef(HighlightedRowName) : nullptr;

	if (!RowPtr || !TablePtr->RowStruct)
		return;

	const FScopedTransaction Transaction(LOCTEXT("PasteDataTableRow", "Paste Data Table Row"));
	TablePtr->Modify();

	FString ClipboardValue;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardValue);

	FDataTableEditorUtils::BroadcastPreChange(TablePtr, FDataTableEditorUtils::EDataTableChangeInfo::RowData);

	const TCHAR* Result = nullptr;

	if (!ClipboardValue.IsEmpty())
	{
		ClipboardValue.TrimEndInline();

		// It is a reasonably safe assumption that if the entire contents is in parentheses, it's in Unreal's own format
		if (ClipboardValue[0] == TCHAR('(') && ClipboardValue[ClipboardValue.Len() - 1] == TCHAR(')'))
		{
			Result = TablePtr->RowStruct->ImportText(*ClipboardValue, RowPtr, TablePtr, PPF_Copy, GWarn, GetPathNameSafe(TablePtr->RowStruct));
		}
		// Otherwise, we'll assume it's from a spreadsheet
		else
		{
			FName RowName = HighlightedRowName;
			if (USheetSerializationSubSystem::Get()->ImportRowFromSheet(ClipboardValue, TablePtr, RowName, RowPtr, GWarn))
			{
				Result = *ClipboardValue;
				if (RowName != HighlightedRowName)
				{
					if (!TablePtr->GetRowMap().Contains(RowName))
					{
						if (FDataTableEditorUtils::RenameRow(TablePtr, HighlightedRowName, RowName))
						{
							SetHighlightedRow(RowName);
						}
					}
					else
					{
						FNotificationInfo Info(LOCTEXT("RowNameAlreadyExists", "Unable to rename row. Name already in use."));
						FSlateNotificationManager::Get().AddNotification(Info);
					}
				}
			}
		}
	}

	TablePtr->HandleDataTableChanged(HighlightedRowName);
	TablePtr->MarkPackageDirty();

	FDataTableEditorUtils::BroadcastPostChange(TablePtr, FDataTableEditorUtils::EDataTableChangeInfo::RowData);

	if (Result == nullptr)
	{
		FNotificationInfo Info(LOCTEXT("FailedPaste", "Failed to paste row"));
		FSlateNotificationManager::Get().AddNotification(Info);
	}
}

void FDataTableEditor::DuplicateSelectedRow()
{
	UDataTable* TablePtr = Cast<UDataTable>(GetEditingObject());
	FName NewName = HighlightedRowName;

	if (NewName == NAME_None || TablePtr == nullptr)
	{
		return;
	}

	const TArray<FName> ExistingNames = TablePtr->GetRowNames();
	while (ExistingNames.Contains(NewName))
	{
		NewName.SetNumber(NewName.GetNumber() + 1);
	}

	FDataTableEditorUtils::DuplicateRow(TablePtr, HighlightedRowName, NewName);
	FDataTableEditorUtils::SelectRow(TablePtr, NewName);
}

void FDataTableEditor::RenameSelectedRowCommand()
{
	UDataTable* TablePtr = Cast<UDataTable>(GetEditingObject());
	FName NewName = HighlightedRowName; 

	if (NewName == NAME_None || TablePtr == nullptr)
	{
		return;
	}

	if (VisibleRows.IsValidIndex(HighlightedVisibleRowIndex))
	{
		TSharedPtr< SDataTableListViewRow > RowWidget = StaticCastSharedPtr< SDataTableListViewRow >(CellsListView->WidgetFromItem(VisibleRows[HighlightedVisibleRowIndex]));
		RowWidget->SetRowForRename();
	}
}

void FDataTableEditor::DeleteSelectedRow()
{
	if (UDataTable* Table = GetEditableDataTable())
	{
		// We must perform this before removing the row
		const int32 RowToRemoveIndex = VisibleRows.IndexOfByPredicate([&](const FDataTableEditorRowListViewDataPtr& InRowName) -> bool
		{
			return InRowName->RowId == HighlightedRowName;
		});
		// Remove row
		if (FDataTableEditorUtils::RemoveRow(Table, HighlightedRowName))
		{
			// Try and keep the same row index selected
			const int32 RowIndexToSelect = FMath::Clamp(RowToRemoveIndex, 0, VisibleRows.Num() - 1);
			if (VisibleRows.IsValidIndex(RowIndexToSelect))
			{
				FDataTableEditorUtils::SelectRow(Table, VisibleRows[RowIndexToSelect]->RowId);
			}
			// Refresh list. Otherwise, the removed row would still appear in the screen until the next list refresh. An
			// analog of CellsListView->RequestListRefresh() also occurs inside FDataTableEditorUtils::SelectRow
			else
			{
				CellsListView->RequestListRefresh();
			}
		}
	}
}

FText FDataTableEditor::GetFilterText() const
{
	return ActiveFilterText;
}

void FDataTableEditor::OnFilterTextChanged(const FText& InFilterText)
{
	ActiveFilterText = InFilterText;
	UpdateVisibleRows();
}

void FDataTableEditor::OnFilterTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnCleared)
	{
		SearchBoxWidget->SetText(FText::GetEmpty());
		OnFilterTextChanged(FText::GetEmpty());
	}
}

void FDataTableEditor::PostRegenerateMenusAndToolbars()
{
	const UDataTable* DataTable = GetDataTable();

	if (DataTable)
	{
		const UUserDefinedStruct* UDS = Cast<const UUserDefinedStruct>(DataTable->GetRowStruct());

		// build and attach the menu overlay
		TSharedRef<SHorizontalBox> MenuOverlayBox = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.ShadowOffset(FVector2D::UnitVector)
				.Text(LOCTEXT("DataTableEditor_RowStructType", "Row Type: "))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SHyperlink)
				.Style(FAppStyle::Get(), "Common.GotoNativeCodeHyperlink")
				.OnNavigate(this, &FDataTableEditor::OnEditDataTableStructClicked)
				.Text(FText::FromName(DataTable->GetRowStructPathName().GetAssetName()))
				.ToolTipText(LOCTEXT("DataTableRowToolTip", "Open the struct used for each row in this data table"))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.VAlign(VAlign_Center)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.OnClicked(this, &FDataTableEditor::OnFindRowInContentBrowserClicked)
				.Visibility(UDS ? EVisibility::Visible : EVisibility::Collapsed)
				.ToolTipText(LOCTEXT("FindRowInCBToolTip", "Find struct in Content Browser"))
				.ContentPadding(4.0f)
				.ForegroundColor(FSlateColor::UseForeground())
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Search"))
				]
			];
	
		SetMenuOverlay(MenuOverlayBox);
	}
}

FReply FDataTableEditor::OnFindRowInContentBrowserClicked()
{
	const UDataTable* DataTable = GetDataTable();
	if(DataTable)
	{
		TArray<FAssetData> ObjectsToSync;
		ObjectsToSync.Add(FAssetData(DataTable->GetRowStruct()));
		GEditor->SyncBrowserToObjects(ObjectsToSync);
	}

	return FReply::Handled();
}

void FDataTableEditor::RefreshCachedDataTable(const FName InCachedSelection, const bool bUpdateEvenIfValid)
{
	UDataTable* Table = GetEditableDataTable();
	TArray<FDataTableEditorColumnHeaderDataPtr> PreviousColumns = AvailableColumns;

	FDataTableEditorUtils::CacheDataTableForEditing(Table, AvailableColumns, AvailableRows);

	// Update the desired width of the row names and numbers column
	// This prevents it growing or shrinking as you scroll the list view
	RefreshRowNumberColumnWidth();
	RefreshRowNameColumnWidth();

	// Setup the default auto-sized columns
	ColumnWidths.SetNum(AvailableColumns.Num());
	for (int32 ColumnIndex = 0; ColumnIndex < AvailableColumns.Num(); ++ColumnIndex)
	{
		const FDataTableEditorColumnHeaderDataPtr& ColumnData = AvailableColumns[ColumnIndex];
		FColumnWidth& ColumnWidth = ColumnWidths[ColumnIndex];
		ColumnWidth.CurrentWidth = FMath::Clamp(ColumnData->DesiredColumnWidth, 10.0f, 400.0f); // Clamp auto-sized columns to a reasonable limit
	}

	// Load the persistent column widths from the layout data
	{
		const TSharedPtr<FJsonObject>* LayoutColumnWidths = nullptr;
		if (LayoutData.IsValid() && LayoutData->TryGetObjectField(TEXT("ColumnWidths"), LayoutColumnWidths))
		{
			for(int32 ColumnIndex = 0; ColumnIndex < AvailableColumns.Num(); ++ColumnIndex)
			{
				const FDataTableEditorColumnHeaderDataPtr& ColumnData = AvailableColumns[ColumnIndex];

				double LayoutColumnWidth = 0.0f;
				if ((*LayoutColumnWidths)->TryGetNumberField(ColumnData->ColumnId.ToString(), LayoutColumnWidth))
				{
					FColumnWidth& ColumnWidth = ColumnWidths[ColumnIndex];
					ColumnWidth.bIsAutoSized = false;
					ColumnWidth.CurrentWidth = static_cast<float>(LayoutColumnWidth);
				}
			}
		}
	}

	if (PreviousColumns != AvailableColumns)
	{
		ColumnNamesHeaderRow->ClearColumns();

		if (CanEditRows())
		{
			ColumnNamesHeaderRow->AddColumn(
				SHeaderRow::Column(RowDragDropColumnId)
				[
					SNew(SBox)
					.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				.ToolTip(IDocumentation::Get()->CreateToolTip(
					LOCTEXT("DataTableRowHandleTooltip", "Drag Drop Handles"),
					nullptr,
					*FDataTableEditorUtils::VariableTypesTooltipDocLink,
					TEXT("DataTableRowHandle")))
				[
					SNew(STextBlock)
					.Text(FText::GetEmpty())
				]
				]
			);
		}	

		ColumnNamesHeaderRow->AddColumn(
			SHeaderRow::Column(RowNumberColumnId)
			.SortMode(this, &FDataTableEditor::GetColumnSortMode, RowNumberColumnId)
			.OnSort(this, &FDataTableEditor::OnColumnNumberSortModeChanged)
			.ManualWidth(this, &FDataTableEditor::GetRowNumberColumnWidth)
			.OnWidthChanged(this, &FDataTableEditor::OnRowNumberColumnResized)
			[
				SNew(SBox)
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				.ToolTip(IDocumentation::Get()->CreateToolTip(
				LOCTEXT("DataTableRowIndexTooltip", "Row Index"),
				nullptr,
				*FDataTableEditorUtils::VariableTypesTooltipDocLink,
				TEXT("DataTableRowIndex")))
				[
					SNew(STextBlock)
					.Text(FText::GetEmpty())
				]
			]

		);

		ColumnNamesHeaderRow->AddColumn(
			SHeaderRow::Column(RowNameColumnId)
			.DefaultLabel(LOCTEXT("DataTableRowName", "Row Name"))
			.ManualWidth(this, &FDataTableEditor::GetRowNameColumnWidth)
			.OnWidthChanged(this, &FDataTableEditor::OnRowNameColumnResized)
			.SortMode(this, &FDataTableEditor::GetColumnSortMode, RowNameColumnId)
			.OnSort(this, &FDataTableEditor::OnColumnNameSortModeChanged)
		);

		for (int32 ColumnIndex = 0; ColumnIndex < AvailableColumns.Num(); ++ColumnIndex)
		{
			const FDataTableEditorColumnHeaderDataPtr& ColumnData = AvailableColumns[ColumnIndex];

			ColumnNamesHeaderRow->AddColumn(
				SHeaderRow::Column(ColumnData->ColumnId)
				.DefaultLabel(ColumnData->DisplayName)
				.ManualWidth(TAttribute<float>::Create(TAttribute<float>::FGetter::CreateSP(this, &FDataTableEditor::GetColumnWidth, ColumnIndex)))
				.OnWidthChanged(this, &FDataTableEditor::OnColumnResized, ColumnIndex)
				.SortMode(this, &FDataTableEditor::GetColumnSortMode, ColumnData->ColumnId)
				.OnSort(this, &FDataTableEditor::OnColumnSortModeChanged)
				[
					SNew(SBox)
					.Padding(FMargin(0, 4, 0, 4))
					.VAlign(VAlign_Fill)
					.ToolTip(IDocumentation::Get()->CreateToolTip(FDataTableEditorUtils::GetRowTypeInfoTooltipText(ColumnData), nullptr, *FDataTableEditorUtils::VariableTypesTooltipDocLink, FDataTableEditorUtils::GetRowTypeTooltipDocExcerptName(ColumnData)))
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.Text(ColumnData->DisplayName)
					]
				]
			);
		}
	}

	UpdateVisibleRows(InCachedSelection, bUpdateEvenIfValid);

	if (PropertyView.IsValid())
	{
		PropertyView->SetObject(Table);
	}
}

void FDataTableEditor::UpdateVisibleRows(const FName InCachedSelection, const bool bUpdateEvenIfValid)
{
	if (ActiveFilterText.IsEmptyOrWhitespace())
	{
		VisibleRows = AvailableRows;
	}
	else
	{
		VisibleRows.Empty(AvailableRows.Num());

		const FString& ActiveFilterString = ActiveFilterText.ToString();
		for (const FDataTableEditorRowListViewDataPtr& RowData : AvailableRows)
		{
			bool bPassesFilter = false;

			if (RowData->DisplayName.ToString().Contains(ActiveFilterString))
			{
				bPassesFilter = true;
			}
			else
			{
				for (const FText& CellText : RowData->CellData)
				{
					if (CellText.ToString().Contains(ActiveFilterString))
					{
						bPassesFilter = true;
						break;
					}
				}
			}

			if (bPassesFilter)
			{
				VisibleRows.Add(RowData);
			}
		}
	}

	CellsListView->RequestListRefresh();
	RestoreCachedSelection(InCachedSelection, bUpdateEvenIfValid);
}

void FDataTableEditor::RestoreCachedSelection(const FName InCachedSelection, const bool bUpdateEvenIfValid)
{
	// Validate the requested selection to see if it matches a known row
	bool bSelectedRowIsValid = false;
	if (!InCachedSelection.IsNone())
	{
		bSelectedRowIsValid = VisibleRows.ContainsByPredicate([&InCachedSelection](const FDataTableEditorRowListViewDataPtr& RowData) -> bool
		{
			return RowData->RowId == InCachedSelection;
		});
	}

	// Apply the new selection (if required)
	if (!bSelectedRowIsValid)
	{
		SetHighlightedRow((VisibleRows.Num() > 0) ? VisibleRows[0]->RowId : NAME_None);
		CallbackOnRowHighlighted.ExecuteIfBound(HighlightedRowName);
	}
	else if (bUpdateEvenIfValid)
	{
		SetHighlightedRow(InCachedSelection);
		CallbackOnRowHighlighted.ExecuteIfBound(HighlightedRowName);
	}
}

TSharedRef<SVerticalBox> FDataTableEditor::CreateContentBox()
{
	TSharedRef<SScrollBar> HorizontalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Horizontal)
		.Thickness(FVector2D(12.0f, 12.0f));

	TSharedRef<SScrollBar> VerticalScrollBar = SNew(SScrollBar)
		.Orientation(Orient_Vertical)
		.Thickness(FVector2D(12.0f, 12.0f));

	ColumnNamesHeaderRow = SNew(SHeaderRow);

	CellsListView = SNew(SListView<FDataTableEditorRowListViewDataPtr>)
		.ListItemsSource(&VisibleRows)
		.HeaderRow(ColumnNamesHeaderRow)
		.OnGenerateRow(this, &FDataTableEditor::MakeRowWidget)
		.OnSelectionChanged(this, &FDataTableEditor::OnRowSelectionChanged)
		.ExternalScrollbar(VerticalScrollBar)
		.ConsumeMouseWheel(EConsumeMouseWheel::Always)
		.SelectionMode(ESelectionMode::Single)
		.AllowOverscroll(EAllowOverscroll::No);

	LoadLayoutData();
	RefreshCachedDataTable();

	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SAssignNew(SearchBoxWidget, SSearchBox)
				.InitialText(this, &FDataTableEditor::GetFilterText)
				.OnTextChanged(this, &FDataTableEditor::OnFilterTextChanged)
				.OnTextCommitted(this, &FDataTableEditor::OnFilterTextCommitted)
			]
		]
		+SVerticalBox::Slot()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				SNew(SScrollBox)
				.Orientation(Orient_Horizontal)
				.ExternalScrollbar(HorizontalScrollBar)
				+SScrollBox::Slot()
				[
					CellsListView.ToSharedRef()
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				VerticalScrollBar
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				HorizontalScrollBar
			]
		];
}

TSharedRef<SWidget> FDataTableEditor::CreateRowEditorBox()
{
	UDataTable* Table = Cast<UDataTable>(GetEditingObject());

	// Support undo/redo
	if (Table)
	{
		Table->SetFlags(RF_Transactional);
	}

	auto RowEditor = SNew(SRowEditor, Table);
	RowEditor->RowSelectedCallback.BindSP(this, &FDataTableEditor::SetHighlightedRow);
	CallbackOnRowHighlighted.BindSP(RowEditor, &SRowEditor::SelectRow);
	CallbackOnDataTableUndoRedo.BindSP(RowEditor, &SRowEditor::HandleUndoRedo);
	return RowEditor;
}

TSharedRef<SRowEditor> FDataTableEditor::CreateRowEditor(UDataTable* Table)
{
	return SNew(SRowEditor, Table);
}

TSharedRef<SDockTab> FDataTableEditor::SpawnTab_RowEditor(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == RowEditorTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("RowEditorTitle", "Row Editor"))
		.TabColorScale(GetTabColorScale())
		[
			SNew(SBorder)
			.Padding(2)
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Fill)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				RowEditorTabWidget.ToSharedRef()
			]
		];
}


TSharedRef<SDockTab> FDataTableEditor::SpawnTab_DataTable( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId().TabType == DataTableTabId );

	UDataTable* Table = Cast<UDataTable>(GetEditingObject());

	// Support undo/redo
	if (Table)
	{
		Table->SetFlags(RF_Transactional);
	}

	return SNew(SDockTab)
		.Label( LOCTEXT("DataTableTitle", "Data Table") )
		.TabColorScale( GetTabColorScale() )
		[
			SNew(SBorder)
			.Padding(2)
			.BorderImage( FAppStyle::GetBrush( "ToolPanel.GroupBorder" ) )
			[
				DataTableTabWidget.ToSharedRef()
			]
		];
}

TSharedRef<SDockTab> FDataTableEditor::SpawnTab_DataTableDetails(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == DataTableDetailsTabId);

	PropertyView->SetObject(GetEditableDataTable());

	return SNew(SDockTab)
		.Label(LOCTEXT("DataTableDetails", "Data Table Details"))
		.TabColorScale(GetTabColorScale())
		[
			SNew(SBorder)
			.Padding(2)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				PropertyView.ToSharedRef()
			]
		];
}

void FDataTableEditor::SetHighlightedRow(FName Name)
{
	if (Name == HighlightedRowName)
	{
		return;
	}

	if (Name.IsNone())
	{
		HighlightedRowName = NAME_None;
		CellsListView->ClearSelection();
		HighlightedVisibleRowIndex = INDEX_NONE;
	}
	else
	{
		HighlightedRowName = Name;

		FDataTableEditorRowListViewDataPtr* NewSelectionPtr = NULL;
		for (HighlightedVisibleRowIndex = 0; HighlightedVisibleRowIndex < VisibleRows.Num(); ++HighlightedVisibleRowIndex)
		{
			if (VisibleRows[HighlightedVisibleRowIndex]->RowId == Name)
			{
				NewSelectionPtr = &(VisibleRows[HighlightedVisibleRowIndex]);

				break;
			}
		}


		// Synchronize the list views
		if (NewSelectionPtr)
		{
			CellsListView->SetSelection(*NewSelectionPtr);

			CellsListView->RequestScrollIntoView(*NewSelectionPtr);
		}
		else
		{
			CellsListView->ClearSelection();
		}
	}
}

#undef LOCTEXT_NAMESPACE
