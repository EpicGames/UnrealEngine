// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "DataTableEditorUtils.h"
#include "Delegates/Delegate.h"
#include "EditorUndoClient.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDataTableEditor.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Kismet2/StructureEditorUtils.h"
#include "Math/Color.h"
#include "Misc/Optional.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Toolkits/IToolkit.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"

class FExtender;
class FJsonObject;
class FSpawnTabArgs;
class FToolBarBuilder;
class ITableRow;
class SDockTab;
class SRowEditor;
class SSearchBox;
class STableViewBase;
class SVerticalBox;
class SWidget;
class UDataTable;

DECLARE_DELEGATE_OneParam(FOnRowHighlighted, FName /*Row name*/);

/** Viewer/editor for a DataTable */
class FDataTableEditor : public IDataTableEditor
	, public FEditorUndoClient
	, public FStructureEditorUtils::INotifyOnStructChanged
	, public FDataTableEditorUtils::INotifyOnDataTableChanged
{
	friend class SDataTableListViewRow;

public:

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	/**
	 * Edits the specified table
	 *
	 * @param	Mode					Asset editing mode for this editor (standalone or world-centric)
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	Table					The table to edit
	 */
	virtual void InitDataTableEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UDataTable* Table );

	virtual bool CanEditRows() const;

	/** Constructor */
	FDataTableEditor();

	/** Destructor */
	virtual ~FDataTableEditor();

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	// FEditorUndoClient
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	void HandleUndoRedo();

	// INotifyOnStructChanged
	virtual void PreChange(const class UUserDefinedStruct* Struct, FStructureEditorUtils::EStructureEditorChangeInfo Info) override;
	virtual void PostChange(const class UUserDefinedStruct* Struct, FStructureEditorUtils::EStructureEditorChangeInfo Info) override;

	// INotifyOnDataTableChanged
	virtual void PreChange(const UDataTable* Changed, FDataTableEditorUtils::EDataTableChangeInfo Info) override;
	virtual void PostChange(const UDataTable* Changed, FDataTableEditorUtils::EDataTableChangeInfo Info) override;
	virtual void SelectionChange(const UDataTable* Changed, FName RowName) override;

	/** Get the data table being edited */
	const UDataTable* GetDataTable() const;

	void HandlePostChange();

	void SetHighlightedRow(FName Name);

	FText GetFilterText() const;

	FSlateColor GetRowTextColor(FName RowName) const;

protected:

	void RefreshCachedDataTable(const FName InCachedSelection = NAME_None, const bool bUpdateEvenIfValid = false);

	void UpdateVisibleRows(const FName InCachedSelection = NAME_None, const bool bUpdateEvenIfValid = false);

	void RestoreCachedSelection(const FName InCachedSelection, const bool bUpdateEvenIfValid = false);
	
	void OnFilterTextChanged(const FText& InFilterText);
	void OnFilterTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);

	virtual void PostRegenerateMenusAndToolbars() override;

	FReply OnFindRowInContentBrowserClicked();

	FText GetCellText(FDataTableEditorRowListViewDataPtr InRowDataPointer, int32 ColumnIndex) const;
	FText GetCellToolTipText(FDataTableEditorRowListViewDataPtr InRowDataPointer, int32 ColumnIndex) const;

	TSharedRef<SVerticalBox> CreateContentBox();

	TSharedRef<SWidget> CreateRowEditorBox();
	virtual TSharedRef<SRowEditor> CreateRowEditor(UDataTable* Table);

	/**	Spawns the tab with the data table inside */
	TSharedRef<SDockTab> SpawnTab_DataTable( const FSpawnTabArgs& Args );

	/**	Spawns the tab with the data table inside */
	TSharedRef<SDockTab> SpawnTab_DataTableDetails(const FSpawnTabArgs& Args);

	/**	Spawns the tab with the Row Editor inside */
	TSharedRef<SDockTab> SpawnTab_RowEditor(const FSpawnTabArgs& Args);

	float GetRowNameColumnWidth() const;
	void RefreshRowNameColumnWidth();

	float GetRowNumberColumnWidth() const;
	void RefreshRowNumberColumnWidth();

	float GetColumnWidth(const int32 ColumnIndex) const;

	void OnColumnResized(const float NewWidth, const int32 ColumnIndex);

	void OnRowNameColumnResized(const float NewWidth);

	void OnRowNumberColumnResized(const float NewWidth);

	void LoadLayoutData();

	void SaveLayoutData();

	/** Make the widget for a row entry in the data table row list view */
	TSharedRef<ITableRow> MakeRowWidget(FDataTableEditorRowListViewDataPtr InRowDataPtr, const TSharedRef<STableViewBase>& OwnerTable);

	/** Make the widget for a cell entry in the data table row list view */
	TSharedRef<SWidget> MakeCellWidget(FDataTableEditorRowListViewDataPtr InRowDataPtr, const int32 InRowIndex, const FName& InColumnId);

	void OnRowSelectionChanged(FDataTableEditorRowListViewDataPtr InNewSelection, ESelectInfo::Type InSelectInfo);

	void CopySelectedRow();
	void PasteOnSelectedRow();
	void DuplicateSelectedRow();
	void RenameSelectedRowCommand();
	void DeleteSelectedRow();

	/** Helper function for creating and registering the tab containing the data table data */
	virtual void CreateAndRegisterDataTableTab(const TSharedRef<class FTabManager>& InTabManager);

	/** Helper function for creating and registering the tab containing the data table details */
	virtual void CreateAndRegisterDataTableDetailsTab(const TSharedRef<class FTabManager>& InTabManager);

	/** Helper function for creating and registering the tab containing the row editor */
	virtual void CreateAndRegisterRowEditorTab(const TSharedRef<class FTabManager>& InTabManager);

	virtual FString GetDocumentationLink() const override;
	
	void OnAddClicked();
	void OnRemoveClicked();
	FReply OnMoveRowClicked(FDataTableEditorUtils::ERowMoveDirection MoveDirection);
	FReply OnMoveToExtentClicked(FDataTableEditorUtils::ERowMoveDirection MoveDirection);
	void OnCopyClicked();
	void OnPasteClicked();
	void OnDuplicateClicked();
	void OnCopySheetClicked();
	void OnPasteSheetClicked();
	bool CanEditTable() const;

	void SetDefaultSort();
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);
	void OnColumnNumberSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);
	void OnColumnNameSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

	void OnEditDataTableStructClicked();

	void ExtendToolbar(TSharedPtr<FExtender> Extender);
	void FillToolbar(FToolBarBuilder& ToolbarBuilder);

private:
	UDataTable* GetEditableDataTable() const;

protected:

	/** Struct holding information about the current column widths */
	struct FColumnWidth
	{
		FColumnWidth()
			: bIsAutoSized(true)
			, CurrentWidth(0.0f)
		{
		}

		/** True if this column is being auto-sized rather than sized by the user */
		bool bIsAutoSized;
		/** The width of the column, either sized by the user, or auto-sized */
		float CurrentWidth;
	};

	/** UI for the "Data Table" tab */
	TSharedPtr<SWidget> DataTableTabWidget;

	/** Property viewing widget */
	TSharedPtr<class IDetailsView> PropertyView;

	/** UI for the "Row Editor" tab */
	TSharedPtr<SSearchBox> SearchBoxWidget;

	/** UI for the "Row Editor" tab */
	TSharedPtr<SWidget> RowEditorTabWidget;

	/** Array of the columns that are available for editing */
	TArray<FDataTableEditorColumnHeaderDataPtr> AvailableColumns;

	/** Array of the rows that are available for editing */
	TArray<FDataTableEditorRowListViewDataPtr> AvailableRows;

	/** Array of the rows that match the active filter(s) */
	TArray<FDataTableEditorRowListViewDataPtr> VisibleRows;

	/** Header row containing entries for each column in AvailableColumns */
	TSharedPtr<SHeaderRow> ColumnNamesHeaderRow;

	/** List view responsible for showing the rows in VisibleRows for each entry in AvailableColumns */
	TSharedPtr<SListView<FDataTableEditorRowListViewDataPtr>> CellsListView;

	/** Width of the row name column */
	float RowNameColumnWidth;

	/** Width of the row number column */
	float RowNumberColumnWidth;

	/** Widths of data table cell columns */
	TArray<FColumnWidth> ColumnWidths;

	/** The layout data for the currently loaded data table */
	TSharedPtr<FJsonObject> LayoutData;

	/** The name of the currently selected row */
	FName HighlightedRowName;

	/** The visible row index of the currently selected row */
	int32 HighlightedVisibleRowIndex;

	/** The current filter text applied to the data table */
	FText ActiveFilterText;

	/** Currently selected sorting mode */
	EColumnSortMode::Type SortMode;

	/** Specify which column to sort with */
	FName SortByColumn;

	FOnRowHighlighted CallbackOnRowHighlighted;

	FSimpleDelegate CallbackOnDataTableUndoRedo;

	/**	The tab id for the data table tab */
	static const FName DataTableTabId;

	/**	The tab id for the data table details tab */
	static const FName DataTableDetailsTabId;

	/**	The tab id for the row editor tab */
	static const FName RowEditorTabId;

	/** The column id for the row name list view column */
	static const FName RowNameColumnId;

	/** The column id for the row number list view column */
	static const FName RowNumberColumnId;

	/** The column id for the drag drop column */
	static const FName RowDragDropColumnId;
};
