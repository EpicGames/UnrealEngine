// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTableEditorModule.h"

#include "CompositeDataTableEditor.h"
#include "DataTableEditor.h"
#include "Engine/CompositeDataTable.h"
#include "Engine/DataTable.h"
#include "HAL/Platform.h"
#include "Modules/ModuleManager.h"
#include "Templates/Casts.h"
#include "UObject/NameTypes.h"

class IToolkitHost;

IMPLEMENT_MODULE( FDataTableEditorModule, DataTableEditor );


const FName FDataTableEditorModule::DataTableEditorAppIdentifier( TEXT( "DataTableEditorApp" ) );

void FDataTableEditorModule::StartupModule()
{
	MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
	ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);
}


void FDataTableEditorModule::ShutdownModule()
{
	MenuExtensibilityManager.Reset();
	ToolBarExtensibilityManager.Reset();
}


TSharedRef<IDataTableEditor> FDataTableEditorModule::CreateDataTableEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UDataTable* Table)
{
	if (Cast<UCompositeDataTable>(Table) != nullptr)
	{
		return CreateCompositeDataTableEditor(Mode, InitToolkitHost, Table);
	}

	return CreateStandardDataTableEditor(Mode, InitToolkitHost, Table);
}

TSharedRef<IDataTableEditor> FDataTableEditorModule::CreateStandardDataTableEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UDataTable* Table)
{
	TSharedRef< FDataTableEditor > NewDataTableEditor( new FDataTableEditor() );
	NewDataTableEditor->InitDataTableEditor( Mode, InitToolkitHost, Table );
	return NewDataTableEditor;
}

TSharedRef<IDataTableEditor> FDataTableEditorModule::CreateCompositeDataTableEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UDataTable* Table)
{
	TSharedRef< FCompositeDataTableEditor > NewDataTableEditor(new FCompositeDataTableEditor());
	NewDataTableEditor->InitDataTableEditor(Mode, InitToolkitHost, Table);
	return NewDataTableEditor;
}
