// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "DataTableEditorSettings.generated.h"

#define UE_API DATATABLEEDITOR_API

class UObject;
struct FPropertyChangedEvent;

UCLASS(MinimalAPI, config = EditorSettings)
class UDataTableEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	/** Set to true to format copied data for pasting into a spreadsheet */
	UPROPERTY(EditAnywhere, config, Category = SpreadsheetInteraction)
	bool bCopyAsSpreadsheetCells = false;
};

#undef UE_API