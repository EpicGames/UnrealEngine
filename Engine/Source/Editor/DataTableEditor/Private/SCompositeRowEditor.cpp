// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCompositeRowEditor.h"

class UDataTable;

#define LOCTEXT_NAMESPACE "SCompositeRowEditor"

SCompositeRowEditor::SCompositeRowEditor()
	: SRowEditor()
{
}

SCompositeRowEditor::~SCompositeRowEditor()
{
}

void SCompositeRowEditor::Construct(const FArguments& InArgs, UDataTable* Changed)
{
	ConstructInternal(Changed);
}

FReply SCompositeRowEditor::OnAddClicked()
{
	return SRowEditor::OnAddClicked();
}

FReply SCompositeRowEditor::OnRemoveClicked()
{
	return SRowEditor::OnRemoveClicked();
}

FReply SCompositeRowEditor::OnMoveRowClicked(FDataTableEditorUtils::ERowMoveDirection MoveDirection)
{
	return SRowEditor::OnMoveRowClicked(MoveDirection);
}

bool SCompositeRowEditor::IsMoveRowUpEnabled() const
{
	return false;
}

bool SCompositeRowEditor::IsMoveRowDownEnabled() const
{
	return false;
}

bool SCompositeRowEditor::IsAddRowEnabled() const
{
	return false;
}

bool SCompositeRowEditor::IsRemoveRowEnabled() const
{
	return false;
}

EVisibility SCompositeRowEditor::GetRenameVisibility() const
{
	return EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
