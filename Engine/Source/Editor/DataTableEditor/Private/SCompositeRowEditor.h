// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataTableEditorUtils.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "SRowEditor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UDataTable;


class SCompositeRowEditor : public SRowEditor
{
	SLATE_BEGIN_ARGS(SRowEditor) {}
	SLATE_END_ARGS()

	SCompositeRowEditor();
	virtual ~SCompositeRowEditor();

	void Construct(const FArguments& InArgs, UDataTable* Changed);

protected:
	virtual FReply OnAddClicked() override;
	virtual FReply OnRemoveClicked() override;
	virtual FReply OnMoveRowClicked(FDataTableEditorUtils::ERowMoveDirection MoveDirection) override;

	virtual bool IsMoveRowUpEnabled() const override;
	virtual bool IsMoveRowDownEnabled() const override;
	virtual bool IsAddRowEnabled() const override;
	virtual bool IsRemoveRowEnabled() const override;
	virtual EVisibility GetRenameVisibility() const override;
};
