// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDataTableListViewRow.h"

#include "DataTableRowMenuContext.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "AssetRegistry/AssetData.h"
#include "Containers/Map.h"
#include "DataTableEditor.h"
#include "DataTableUtils.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Engine/DataTable.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "Input/Events.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/WidgetPath.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/MessageDialog.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"

class STableViewBase;
class SWidget;
struct FGeometry;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "SDataTableListViewRowName"

void SDataTableListViewRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	RowDataPtr = InArgs._RowDataPtr;
	CurrentName = MakeShareable(new FName(RowDataPtr->RowId));
	DataTableEditor = InArgs._DataTableEditor;
	IsEditable = InArgs._IsEditable;
	SMultiColumnTableRow<FDataTableEditorRowListViewDataPtr>::Construct(
		FSuperRowType::FArguments()
		.Style(FAppStyle::Get(), "DataTableEditor.CellListViewRow")
		.OnDrop(this, &SDataTableListViewRow::OnRowDrop)
		.OnDragEnter(this, &SDataTableListViewRow::OnRowDragEnter)
		.OnDragLeave(this, &SDataTableListViewRow::OnRowDragLeave),
		InOwnerTableView
	);

	SetBorderImage(TAttribute<const FSlateBrush*>(this, &SDataTableListViewRow::GetBorder));

}

FReply SDataTableListViewRow::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (IsEditable && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && RowDataPtr.IsValid() && DataTableEditor.IsValid())
	{
		FDataTableEditorUtils::SelectRow(DataTableEditor.Pin()->GetDataTable(), RowDataPtr->RowId);

		TSharedRef<SWidget> MenuWidget = MakeRowActionsMenu();

		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
		FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuWidget, MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect::ContextMenu);
		return FReply::Handled();
	}

	return STableRow::OnMouseButtonUp(MyGeometry, MouseEvent);
}


FReply SDataTableListViewRow::OnRowDrop(const FDragDropEvent& DragDropEvent)
{
	bIsHoveredDragTarget = false;

	TSharedPtr<FDataTableRowDragDropOp> DataTableDropOp = DragDropEvent.GetOperationAs< FDataTableRowDragDropOp >();
	TSharedPtr<SDataTableListViewRow> RowPtr = nullptr;
	if (DataTableDropOp.IsValid() && DataTableDropOp->Row.IsValid())
	{
		RowPtr = DataTableDropOp->Row.Pin();
	}
	if (!RowPtr.IsValid())
	{
		return FReply::Unhandled();
	}

	int32 JumpCount = (RowPtr->RowDataPtr)->RowNum - RowDataPtr->RowNum;

	if (!JumpCount)
	{
		return FReply::Handled();
	}

	FDataTableEditorUtils::ERowMoveDirection Direction = JumpCount > 0 ? FDataTableEditorUtils::ERowMoveDirection::Up : FDataTableEditorUtils::ERowMoveDirection::Down;

	if (FDataTableEditor* DataTableEditorPtr = DataTableEditor.Pin().Get())
	{
		UDataTable* SourceDataTable = const_cast<UDataTable*>(DataTableEditorPtr->GetDataTable());

		if (SourceDataTable)
		{
			FName& RowId = (RowPtr->RowDataPtr)->RowId;

			FDataTableEditorUtils::MoveRow(SourceDataTable, RowId, Direction, FMath::Abs<int32>(JumpCount));
			
			FDataTableEditorUtils::SelectRow(SourceDataTable, RowId);

			DataTableEditorPtr->SortMode = EColumnSortMode::Ascending;
			DataTableEditorPtr->SortByColumn = FDataTableEditor::RowNumberColumnId;

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply SDataTableListViewRow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FKey Key = InKeyEvent.GetKey();

	if (Key == EKeys::Escape && InlineEditableText->HasKeyboardFocus())
	{
		// Clear focus
		return FReply::Handled().SetUserFocus(SharedThis(this), EFocusCause::Cleared);
	}

	return FReply::Unhandled();
}

void SDataTableListViewRow::OnRowRenamed(const FText& Text, ETextCommit::Type CommitType)
{
	UDataTable* DataTable = Cast<UDataTable>(DataTableEditor.Pin()->GetEditingObject());

	if (!GetCurrentNameAsText().EqualTo(Text) && DataTable)
	{
		if (FDataTableEditor* DataTableEditorPtr = DataTableEditor.Pin().Get())
		{

			const TArray<FName> RowNames = DataTable->GetRowNames();

			if (Text.IsEmptyOrWhitespace() || !FName::IsValidXName(Text.ToString(), INVALID_NAME_CHARACTERS))
			{
				// Only pop up the error dialog if the rename was caused by the user's action
				if ((CommitType == ETextCommit::OnEnter) || (CommitType == ETextCommit::OnUserMovedFocus))
				{
					// popup an error dialog here
					const FText Message = FText::Format(LOCTEXT("InvalidRowName", "'{0}' is not a valid row name"), Text);
					FMessageDialog::Open(EAppMsgType::Ok, Message);
				}
				return;
			}
			const FName NewName = DataTableUtils::MakeValidName(Text.ToString());
			if (NewName == NAME_None)
			{
				// popup an error dialog here
				const FText Message = FText::Format(LOCTEXT("InvalidRowName", "'{0}' is not a valid row name"), Text);
				FMessageDialog::Open(EAppMsgType::Ok, Message);

				return;
			}
			for (const FName& Name : RowNames)
			{
				if (Name.IsValid() && (Name == NewName))
				{
					//the name already exists
					// popup an error dialog here
					const FText Message = FText::Format(LOCTEXT("DuplicateRowName", "'{0}' is already used as a row name in this table"), Text);
					FMessageDialog::Open(EAppMsgType::Ok, Message);
					return;

				}
			}

			const FName OldName = GetCurrentName();
			FDataTableEditorUtils::RenameRow(DataTable, OldName, NewName);
			FDataTableEditorUtils::SelectRow(DataTable, NewName);

			DataTableEditorPtr->SetDefaultSort();

			*CurrentName = NewName;
		}
	}
}

TSharedRef<SWidget> SDataTableListViewRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedPtr<FDataTableEditor> DataTableEditorPtr = DataTableEditor.Pin();
	return (DataTableEditorPtr.IsValid())
		? MakeCellWidget(IndexInList, ColumnName)
		: SNullWidget::NullWidget;
}

TSharedRef<SWidget> SDataTableListViewRow::MakeCellWidget(const int32 InRowIndex, const FName& InColumnId)
{
	const FName RowDragDropColumnId("RowDragDrop");

	int32 ColumnIndex = 0;

	FDataTableEditor* DataTableEdit = DataTableEditor.Pin().Get();
	TArray<FDataTableEditorColumnHeaderDataPtr>& AvailableColumns = DataTableEdit->AvailableColumns;

	if (InColumnId.IsEqual(RowDragDropColumnId))
	{
		return SNew(SDataTableRowHandle)
			.Content()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(5.0f, 1.0f)
				[
					SNew(SImage)
					.Image(FCoreStyle::Get().GetBrush("VerticalBoxDragIndicatorShort"))
				]
			]
			.ParentRow(SharedThis(this));
	}

	const FName RowNumberColumnId("RowNumber");

	if (InColumnId.IsEqual(RowNumberColumnId))
	{
		return SNew(SBox)
			.Padding(FMargin(4, 2, 4, 2))
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "DataTableEditor.CellText")
				.Text(FText::FromString(FString::FromInt(RowDataPtr->RowNum)))
				.ColorAndOpacity(DataTableEdit, &FDataTableEditor::GetRowTextColor, RowDataPtr->RowId)
				.HighlightText(DataTableEdit, &FDataTableEditor::GetFilterText)
			];
	}

	const FName RowNameColumnId("RowName");

	if (InColumnId.IsEqual(RowNameColumnId))
	{
		return SNew(SBox)
			.Padding(FMargin(4, 2, 4, 2))
			[
				SAssignNew(InlineEditableText, SInlineEditableTextBlock)
				.Text(RowDataPtr->DisplayName)
				.OnTextCommitted(this, &SDataTableListViewRow::OnRowRenamed)
				.HighlightText(DataTableEdit, &FDataTableEditor::GetFilterText)
				.ColorAndOpacity(DataTableEdit, &FDataTableEditor::GetRowTextColor, RowDataPtr->RowId)
				.IsReadOnly(!IsEditable)
			];
	}

	for (; ColumnIndex < AvailableColumns.Num(); ++ColumnIndex)
	{
		const FDataTableEditorColumnHeaderDataPtr& ColumnData = AvailableColumns[ColumnIndex];
		if (ColumnData->ColumnId == InColumnId)
		{
			break;
		}
	}
	 
	// Valid column ID?
	if (AvailableColumns.IsValidIndex(ColumnIndex) && RowDataPtr->CellData.IsValidIndex(ColumnIndex))
	{
		return SNew(SBox)
			.Padding(FMargin(4, 2, 4, 2))
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "DataTableEditor.CellText")
				.ColorAndOpacity(DataTableEdit, &FDataTableEditor::GetRowTextColor, RowDataPtr->RowId)
				.Text(DataTableEdit, &FDataTableEditor::GetCellText, RowDataPtr, ColumnIndex)
				.HighlightText(DataTableEdit, &FDataTableEditor::GetFilterText)
				.ToolTipText(DataTableEdit, &FDataTableEditor::GetCellToolTipText, RowDataPtr, ColumnIndex)
			];
	}

	return SNullWidget::NullWidget;
}

FName SDataTableListViewRow::GetCurrentName() const
{
	return CurrentName.IsValid() ? *CurrentName : NAME_None;

}

uint32 SDataTableListViewRow::GetCurrentIndex() const
{
	return RowDataPtr.IsValid() ? RowDataPtr->RowNum : -1;
}

const TArray<FText>& SDataTableListViewRow::GetCellValues() const
{
	check(RowDataPtr)
	return RowDataPtr->CellData;
}

FReply SDataTableListViewRow::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (InlineEditableText->IsHovered())
	{
		InlineEditableText->EnterEditingMode();
	}

	return FReply::Handled();
}

void SDataTableListViewRow::SetRowForRename()
{
	InlineEditableText->EnterEditingMode();
}

const FDataTableEditorRowListViewDataPtr& SDataTableListViewRow::GetRowDataPtr() const
{
	return RowDataPtr;
}

FText SDataTableListViewRow::GetCurrentNameAsText() const
{
	return FText::FromName(GetCurrentName());
}

void SDataTableRowHandle::Construct(const FArguments& InArgs)
{
	ParentRow = InArgs._ParentRow;

	ChildSlot
		[
			InArgs._Content.Widget
		];
}

FReply SDataTableRowHandle::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		TSharedPtr<FDragDropOperation> DragDropOp = CreateDragDropOperation(ParentRow.Pin());
		if (DragDropOp.IsValid())
		{
			return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
		}
	}

	return FReply::Unhandled();

}

TSharedPtr<FDataTableRowDragDropOp> SDataTableRowHandle::CreateDragDropOperation(TSharedPtr<SDataTableListViewRow> InRow)
{
	TSharedPtr<FDataTableRowDragDropOp> Operation = MakeShareable(new FDataTableRowDragDropOp(InRow));

	return Operation;
}

void SDataTableListViewRow::SetIsDragDrop(bool bInIsDragDrop)
{
	bIsDragDropObject = bInIsDragDrop;
}


void SDataTableListViewRow::OnRowDragEnter(const FDragDropEvent& DragDropEvent)
{
	bIsHoveredDragTarget = true;
}

void SDataTableListViewRow::OnRowDragLeave(const FDragDropEvent& DragDropEvent)
{
	bIsHoveredDragTarget = false;
}


const FSlateBrush* SDataTableListViewRow::GetBorder() const
{
	if (bIsDragDropObject)
	{
		return FAppStyle::GetBrush("DataTableEditor.DragDropObject");
	}
	else if (bIsHoveredDragTarget)
	{
		return FAppStyle::GetBrush("DataTableEditor.DragDropHoveredTarget");
	}
	else
	{
		return STableRow::GetBorder();
	}
}


TSharedRef<SWidget> SDataTableListViewRow::MakeRowActionsMenu()
{
	static const FName MenuName = "DataTableEditor.RowContextMenu";

	if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);
		Menu->AddDynamicSection("DefaultRowActions", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			// Validate context.
			UDataTableRowMenuContext* Ctx = InMenu->FindContext<UDataTableRowMenuContext>();
			if (!Ctx || !Ctx->Toolkit.IsValid() || !Ctx->RowDataPtr)
			{
				return;
			}

			// Get data table editor.
			TWeakPtr<FDataTableEditor> WeakEditor = StaticCastWeakPtr<FDataTableEditor>(Ctx->Toolkit);
			const FName RowId = Ctx->RowDataPtr->RowId;

			// Insert actions
			FToolMenuSection& InsertSection = InMenu->FindOrAddSection("InsertActions");

			auto MakeInsertAction = [WeakEditor, RowId](ERowInsertionPosition InsertPos) -> FUIAction
			{
				return FUIAction(FExecuteAction::CreateLambda([WeakEditor, RowId, InsertPos]()
				{
					TSharedPtr<FDataTableEditor> Editor = WeakEditor.Pin();
					if (!Editor.IsValid())
					{
						return;
					}
						
					UDataTable* Table = Cast<UDataTable>(Editor->GetEditingObject());
					if (!Table)
					{
						return;
					}

					FName NewName = DataTableUtils::MakeValidName(TEXT("NewRow"));
					while (Table->GetRowMap().Contains(NewName))
					{
						NewName.SetNumber(NewName.GetNumber() + 1);
					}

					if (InsertPos == ERowInsertionPosition::Bottom)
					{
						FDataTableEditorUtils::AddRow(Table, NewName);
					}
					else
					{
						FDataTableEditorUtils::AddRowAboveOrBelowSelection(Table, RowId, NewName, InsertPos);
					}

					FDataTableEditorUtils::SelectRow(Table, NewName);
					Editor->SetDefaultSort();
				}));
			};

			InsertSection.AddMenuEntry("InsertNewRow",
				LOCTEXT("DataTableRowMenuActions_InsertNewRow", "Insert New Row"),
				LOCTEXT("DataTableRowMenuActions_InsertNewRowTooltip", "Insert a new row"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "DataTableEditor.Add"),
				MakeInsertAction(ERowInsertionPosition::Bottom)
			);
			InsertSection.AddMenuEntry("InsertNewRowAbove",
				LOCTEXT("DataTableRowMenuActions_InsertNewRowAbove", "Insert New Row Above"),
				LOCTEXT("DataTableRowMenuActions_InsertNewRowAboveTooltip", "Insert a new Row above the current selection"),
				FSlateIcon(),
				MakeInsertAction(ERowInsertionPosition::Above)
			);
			InsertSection.AddMenuEntry("InsertNewRowBelow",
				LOCTEXT("DataTableRowMenuActions_InsertNewRowBelow", "Insert New Row Below"),
				LOCTEXT("DataTableRowMenuActions_InsertNewRowBelowTooltip", "Insert a new Row below the current selection"),
				FSlateIcon(),
				MakeInsertAction(ERowInsertionPosition::Below)
			);

			// Edit actions — actions are resolved from the command list appended to the context
			FToolMenuSection& EditSection = InMenu->FindOrAddSection("EditActions");
			EditSection.AddEntry(FToolMenuEntry::InitMenuEntry(FGenericCommands::Get().Copy));
			EditSection.AddEntry(FToolMenuEntry::InitMenuEntry(FGenericCommands::Get().Paste));
			EditSection.AddEntry(FToolMenuEntry::InitMenuEntry(FGenericCommands::Get().Duplicate));
			EditSection.AddEntry(FToolMenuEntry::InitMenuEntry(FGenericCommands::Get().Rename));
			EditSection.AddEntry(FToolMenuEntry::InitMenuEntry(FGenericCommands::Get().Delete));
				
			// Move actions
			FToolMenuSection& MoveSection = InMenu->FindOrAddSection("MoveActions");
			MoveSection.AddSeparator("MoveActionsSeparator");
			MoveSection.AddMenuEntry("MoveRowToTop",
				LOCTEXT("DataTableRowMenuActions_MoveToTopAction", "Move Row to Top"),
				LOCTEXT("DataTableRowMenuActions_MoveToTopActionTooltip", "Move selected Row to the top"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Symbols.DoubleUpArrow"),
				FUIAction(FExecuteAction::CreateLambda([WeakEditor]()
				{
					if (TSharedPtr<FDataTableEditor> Editor = WeakEditor.Pin())
					{
						Editor->OnMoveToExtentClicked(FDataTableEditorUtils::ERowMoveDirection::Up);
					}
				}))
			);
			MoveSection.AddMenuEntry("MoveRowToBottom",
				LOCTEXT("DataTableRowMenuActions_MoveToBottom", "Move Row To Bottom"),
				LOCTEXT("DataTableRowMenuActions_MoveToBottomTooltip", "Move selected Row to the bottom"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Symbols.DoubleDownArrow"),
				FUIAction(FExecuteAction::CreateLambda([WeakEditor]()
				{
					if (TSharedPtr<FDataTableEditor> Editor = WeakEditor.Pin())
					{
						Editor->OnMoveToExtentClicked(FDataTableEditorUtils::ERowMoveDirection::Down);
					}
				}))
			);

			// Reference actions
			FToolMenuSection& RefSection = InMenu->FindOrAddSection("ReferenceActions");
			RefSection.AddSeparator("ReferenceActionsSeparator");
			RefSection.AddMenuEntry("FindRowReferences",
				NSLOCTEXT("FDataTableRowUtils", "FDataTableRowUtils_SearchForReferences", "Find Row References"),
				NSLOCTEXT("FDataTableRowUtils", "FDataTableRowUtils_SearchForReferencesTooltip", "Find assets that reference this Row"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([WeakEditor, RowId]()
					{
						TSharedPtr<FDataTableEditor> Editor = WeakEditor.Pin();
						if (!Editor.IsValid())
						{
							return;
						}
							
						UDataTable* Table = Cast<UDataTable>(Editor->GetEditingObject());
						if (!Table)
						{
							return;
						}
							
						TArray<FAssetIdentifier> AssetIdentifiers;
						AssetIdentifiers.Add(FAssetIdentifier(Table, RowId));
						FEditorDelegates::OnOpenReferenceViewer.Broadcast(AssetIdentifiers, FReferenceViewerParams());
					}),
					FCanExecuteAction::CreateLambda([]()
					{
						return FEditorDelegates::OnOpenReferenceViewer.IsBound();
					})
				)
			);
		}));
	}

	UDataTableRowMenuContext* RowContext = NewObject<UDataTableRowMenuContext>();
	RowContext->Toolkit = DataTableEditor;
	RowContext->RowDataPtr = RowDataPtr;

	FToolMenuContext MenuContext(DataTableEditor.Pin()->GetToolkitCommands());
	MenuContext.AddObject(RowContext);

	return UToolMenus::Get()->GenerateWidget(MenuName, MenuContext);
}

FDataTableRowDragDropOp::FDataTableRowDragDropOp(TSharedPtr<SDataTableListViewRow> InRow)
{
	Row = InRow;

	TSharedPtr<SDataTableListViewRow> RowPtr = nullptr;
	if (Row.IsValid())
	{
		RowPtr = Row.Pin();
		RowPtr->SetIsDragDrop(true);

		DecoratorWidget = SNew(SBorder)
			.Padding(8.f)
			.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
			.Content()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::Format(NSLOCTEXT("DataTableDragDrop", "PlaceRowHere", "Place Row {0} Here"), FText::AsNumber(InRow->GetCurrentIndex())))
				]
			];

		Construct();
	}
}

void FDataTableRowDragDropOp::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	FDecoratedDragDropOp::OnDrop(bDropWasHandled, MouseEvent);

	TSharedPtr<SDataTableListViewRow> RowPtr = nullptr;
	if (Row.IsValid())
	{
		RowPtr = Row.Pin();
		RowPtr->SetIsDragDrop(false);
	}
}

#undef LOCTEXT_NAMESPACE
