// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/UnrealString.h"
#include "DataTableEditorUtils.h"
#include "Delegates/IDelegateInstance.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "HAL/Platform.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FDataTableEditor;
class SDataTableListViewRow;
class SInlineEditableTextBlock;
class STableViewBase;
class SWidget;
struct FGeometry;
struct FKeyEvent;
struct FPointerEvent;
struct FSlateBrush;

class SDataTableRowHandle: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDataTableRowHandle)
	{}
	SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_ARGUMENT(TSharedPtr<SDataTableListViewRow>, ParentRow)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	};


	FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	TSharedPtr<class FDataTableRowDragDropOp> CreateDragDropOperation(TSharedPtr<SDataTableListViewRow> InRow);

private:
	TWeakPtr<SDataTableListViewRow> ParentRow;
};

/**
 * A widget to represent a row in a Data Table Editor widget. This widget allows us to do things like right-click
 * and take actions on a particular row of a Data Table.
 */
class SDataTableListViewRow : public SMultiColumnTableRow<FDataTableEditorRowListViewDataPtr>
{
public:

	SLATE_BEGIN_ARGS(SDataTableListViewRow)
		: _IsEditable(true)
	{
	}
	/** The owning object. This allows us access to the actual data table being edited as well as some other API functions. */
	SLATE_ARGUMENT(TSharedPtr<FDataTableEditor>, DataTableEditor)
		/** The row we're working with to allow us to get naming information. */
		SLATE_ARGUMENT(FDataTableEditorRowListViewDataPtr, RowDataPtr)
		SLATE_ARGUMENT(bool, IsEditable)
		SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	void OnRowRenamed(const FText& Text, ETextCommit::Type CommitType);

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	FText GetCurrentNameAsText() const;
	FName GetCurrentName() const;
	uint32 GetCurrentIndex() const;

	const TArray<FText>& GetCellValues() const;

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	void SetRowForRename();

	void SetIsDragDrop(bool bInIsDragDrop);

	const FDataTableEditorRowListViewDataPtr& GetRowDataPtr() const;

private:

	FReply OnRowDrop(const FDragDropEvent& DragDropEvent);

	TSharedRef<SWidget> MakeCellWidget(const int32 InRowIndex, const FName& InColumnId);

	void OnRowDragEnter(const FDragDropEvent& DragDropEvent);
	void OnRowDragLeave(const FDragDropEvent& DragDropEvent);

	virtual const FSlateBrush* GetBorder() const;

	TSharedRef<SWidget> MakeRowActionsMenu();

	TSharedPtr<SInlineEditableTextBlock> InlineEditableText;

	TSharedPtr<FName> CurrentName;

	FDataTableEditorRowListViewDataPtr RowDataPtr;
	TWeakPtr<FDataTableEditor> DataTableEditor;

	bool IsEditable;
	bool bIsDragDropObject;
	bool bIsHoveredDragTarget;
};

class FDataTableRowDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDataTableRowDragDropOp, FDecoratedDragDropOp)

	FDataTableRowDragDropOp(TSharedPtr<SDataTableListViewRow> InRow);

	void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent);

	TSharedPtr<SWidget> DecoratorWidget;

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return DecoratorWidget;
	}

	TWeakPtr<class SDataTableListViewRow> Row;

};
