// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRowEditor.h"

#include "Containers/Map.h"
#include "DataTableUtils.h"
#include "DetailsViewArgs.h"
#include "Engine/DataTable.h"
#include "StructUtils/UserDefinedStruct.h"
#include "HAL/PlatformMisc.h"
#include "IStructureDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Types/SlateStructs.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"
#include "UObject/StructOnScope.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class FProperty;
class SWidget;
class UPackage;
struct FPropertyChangedEvent;

#define LOCTEXT_NAMESPACE "SRowEditor"

class FStructFromDataTable : public FStructOnScope
{
	TWeakObjectPtr<UDataTable> DataTable;
	FName RowName;

public:
	FStructFromDataTable(UDataTable* InDataTable, FName InRowName)
		: FStructOnScope()
		, DataTable(InDataTable)
		, RowName(InRowName)
	{}

	virtual uint8* GetStructMemory() override
	{
		return (DataTable.IsValid() && !RowName.IsNone()) ? DataTable->FindRowUnchecked(RowName) : nullptr;
	}

	virtual const uint8* GetStructMemory() const override
	{
		return (DataTable.IsValid() && !RowName.IsNone()) ? DataTable->FindRowUnchecked(RowName) : nullptr;
	}

	virtual const UScriptStruct* GetStruct() const override
	{
		return DataTable.IsValid() ? DataTable->GetRowStruct() : nullptr;
	}

	virtual UPackage* GetPackage() const override
	{
		return DataTable.IsValid() ? DataTable->GetOutermost() : nullptr;
	}

	virtual void SetPackage(UPackage* InPackage) override
	{
	}

	virtual bool IsValid() const override
	{
		return !RowName.IsNone()
			&& DataTable.IsValid()
			&& DataTable->GetRowStruct()
			&& DataTable->FindRowUnchecked(RowName);
	}

	virtual void Destroy() override
	{
		DataTable = nullptr;
		RowName = NAME_None;
	}

	FName GetRowName() const
	{
		return RowName;
	}
};

SRowEditor::SRowEditor() 
	: SCompoundWidget()
{
}

SRowEditor::~SRowEditor()
{
}

void SRowEditor::NotifyPreChange( FProperty* PropertyAboutToChange )
{
	check(DataTable.IsValid());
	DataTable->Modify();

	FDataTableEditorUtils::BroadcastPreChange(DataTable.Get(), FDataTableEditorUtils::EDataTableChangeInfo::RowData);
}

void SRowEditor::NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged )
{
	check(DataTable.IsValid());

	FName RowName = NAME_None;
	if (SelectedName.IsValid())
	{
		RowName = *SelectedName.Get();
	}

	DataTable->HandleDataTableChanged(RowName);
	DataTable->MarkPackageDirty();

	FDataTableEditorUtils::BroadcastPostChange(DataTable.Get(), FDataTableEditorUtils::EDataTableChangeInfo::RowData);
}

void SRowEditor::PreChange(const class UUserDefinedStruct* Struct, FStructureEditorUtils::EStructureEditorChangeInfo Info)
{
	if (Struct && (GetScriptStruct() == Struct))
	{
		CleanBeforeChange();
	}
}

void SRowEditor::PostChange(const class UUserDefinedStruct* Struct, FStructureEditorUtils::EStructureEditorChangeInfo Info)
{
	if (Struct && (GetScriptStruct() == Struct))
	{
		RefreshNameList();
		Restore();
	}
}

void SRowEditor::PreChange(const UDataTable* Changed, FDataTableEditorUtils::EDataTableChangeInfo Info)
{
	if ((Changed == DataTable.Get()) && (FDataTableEditorUtils::EDataTableChangeInfo::RowList == Info))
	{
		CleanBeforeChange();
	}
}

void SRowEditor::PostChange(const UDataTable* Changed, FDataTableEditorUtils::EDataTableChangeInfo Info)
{
	if ((Changed == DataTable.Get()) && (FDataTableEditorUtils::EDataTableChangeInfo::RowList == Info))
	{
		RefreshNameList();
		Restore();
	}
}

void SRowEditor::CleanBeforeChange()
{
	if (StructureDetailsView.IsValid())
	{
		StructureDetailsView->SetStructureData(NULL);
	}
	if (CurrentRow.IsValid())
	{
		CurrentRow->Destroy();
		CurrentRow.Reset();
	}
}

void SRowEditor::RefreshNameList()
{
	CachedRowNames.Empty();
	if (DataTable.IsValid())
	{
		auto RowNames = DataTable->GetRowNames();
		for (auto RowName : RowNames)
		{
			CachedRowNames.Add(MakeShareable(new FName(RowName)));
		}
	}
}

void SRowEditor::Restore()
{
	if (!SelectedName.IsValid() || !SelectedName->IsNone())
	{
		if (SelectedName.IsValid())
		{
			auto CurrentName = *SelectedName;
			SelectedName = NULL;
			for (auto Element : CachedRowNames)
			{
				if (*Element == CurrentName)
				{
					SelectedName = Element;
					break;
				}
			}
		}

		if (!SelectedName.IsValid() && CachedRowNames.Num() && CachedRowNames[0].IsValid())
		{
			SelectedName = CachedRowNames[0];
		}

		if (RowComboBox.IsValid())
		{
			RowComboBox->SetSelectedItem(SelectedName);
		}
	}
	else
	{
		if (RowComboBox.IsValid())
		{
			RowComboBox->ClearSelection();
		}
	}

	auto FinalName = SelectedName.IsValid() ? *SelectedName : NAME_None;
	CurrentRow = MakeShareable(new FStructFromDataTable(DataTable.Get(), FinalName));
	if (StructureDetailsView.IsValid())
	{
		StructureDetailsView->SetCustomName(FText::FromName(FinalName));
		if (!FinalName.IsNone())
		{
			StructureDetailsView->SetStructureData(CurrentRow);
		}
	}

	RowSelectedCallback.ExecuteIfBound(FinalName);
}

UScriptStruct* SRowEditor::GetScriptStruct() const
{
	return DataTable.IsValid() ? DataTable->RowStruct : nullptr;
}

FName SRowEditor::GetCurrentName() const
{
	return SelectedName.IsValid() ? *SelectedName : NAME_None;
}

FText SRowEditor::GetCurrentNameAsText() const
{
	return FText::FromName(GetCurrentName());
}

FString SRowEditor::GetStructureDisplayName() const
{
	const auto Struct = DataTable.IsValid() ? DataTable->GetRowStruct() : nullptr;
	return Struct 
		? Struct->GetDisplayNameText().ToString()
		: LOCTEXT("Error_UnknownStruct", "Error: Unknown Struct").ToString();
}

TSharedRef<SWidget> SRowEditor::OnGenerateWidget(TSharedPtr<FName> InItem)
{
	return SNew(STextBlock).Text(FText::FromName(InItem.IsValid() ? *InItem : NAME_None));
}

void SRowEditor::OnSelectionChanged(TSharedPtr<FName> InItem, ESelectInfo::Type InSeletionInfo)
{
	if (InItem.IsValid() && InItem != SelectedName)
	{
		CleanBeforeChange();

		SelectedName = InItem;

		Restore();
	}
}

void SRowEditor::SelectRow(FName InName)
{
	TSharedPtr<FName> NewSelectedName;
	for (auto Name : CachedRowNames)
	{
		if (Name.IsValid() && (*Name == InName))
		{
			NewSelectedName = Name;
		}
	}
	if (!NewSelectedName.IsValid())
	{
		NewSelectedName = MakeShareable(new FName(InName));
	}
	OnSelectionChanged(NewSelectedName, ESelectInfo::Direct);
}

void SRowEditor::HandleUndoRedo()
{
	RefreshNameList();
	Restore();
}

FReply SRowEditor::OnAddClicked()
{
	if (DataTable.IsValid())
	{
		FName NewName = DataTableUtils::MakeValidName(TEXT("NewRow"));
		const TArray<FName> ExisitngNames = DataTable->GetRowNames();
		while (ExisitngNames.Contains(NewName))
		{
			NewName.SetNumber(NewName.GetNumber() + 1);
		}
		FDataTableEditorUtils::AddRow(DataTable.Get(), NewName);
		SelectRow(NewName);
	}
	return FReply::Handled();
}

FReply SRowEditor::OnRemoveClicked()
{
	if (DataTable.IsValid())
	{
		const FName RowToRemove = GetCurrentName();
		const int32 RowToRemoveIndex = CachedRowNames.IndexOfByPredicate([&](const TSharedPtr<FName>& InRowName) -> bool
		{
			return *InRowName == RowToRemove;
		});

		if (FDataTableEditorUtils::RemoveRow(DataTable.Get(), RowToRemove))
		{
			// Try and keep the same row index selected
			const int32 RowIndexToSelect = FMath::Clamp(RowToRemoveIndex, 0, CachedRowNames.Num() - 1);
			if (CachedRowNames.IsValidIndex(RowIndexToSelect))
			{
				SelectRow(*CachedRowNames[RowIndexToSelect]);
			}
		}
	}
	return FReply::Handled();
}

FReply SRowEditor::OnMoveRowClicked(FDataTableEditorUtils::ERowMoveDirection MoveDirection)
{
	if (DataTable.IsValid())
	{
		const FName RowToMove = GetCurrentName();
		FDataTableEditorUtils::MoveRow(DataTable.Get(), RowToMove, MoveDirection);
	}
	return FReply::Handled();
}

FReply SRowEditor::OnMoveToExtentClicked(FDataTableEditorUtils::ERowMoveDirection MoveDirection)
{
	if (DataTable.IsValid())
	{
		// We move by the row map size, as FDataTableEditorUtils::MoveRow will automatically clamp this as appropriate
		const FName RowToMove = GetCurrentName();
		FDataTableEditorUtils::MoveRow(DataTable.Get(), RowToMove, MoveDirection, DataTable->GetRowMap().Num());
	}
	return FReply::Handled();
}

void SRowEditor::OnRowRenamed(const FText& Text, ETextCommit::Type CommitType)
{
	if (!GetCurrentNameAsText().EqualTo(Text) && DataTable.IsValid())
	{
		if (Text.IsEmptyOrWhitespace() || !FName::IsValidXName(Text.ToString(), INVALID_NAME_CHARACTERS))
		{
			// Only pop up the error dialog if the rename was caused by the user's action
			if ((CommitType == ETextCommit::OnEnter) || (CommitType == ETextCommit::OnUserMovedFocus ))
			{
				// popup an error dialog here
				const FText Message = FText::Format(LOCTEXT("InvalidRowName", "'{0}' is not a valid row name"), Text);
				FMessageDialog::Open(EAppMsgType::Ok, Message);
			}
			return;
		}
		const FName NewName = DataTableUtils::MakeValidName(Text.ToString());
		if (NewName == NAME_None)
		{
			// popup an error dialog here
			const FText Message = FText::Format(LOCTEXT("InvalidRowName", "'{0}' is not a valid row name"), Text);
			FMessageDialog::Open(EAppMsgType::Ok, Message);

			return;
		}
		for (auto Name : CachedRowNames)
		{
			if (Name.IsValid() && (*Name == NewName))
			{
				//the name already exists
				// popup an error dialog here
				const FText Message = FText::Format(LOCTEXT("DuplicateRowName", "'{0}' is already used as a row name in this table"), Text);
				FMessageDialog::Open(EAppMsgType::Ok, Message);
				return;
			}
		}

		const FName OldName = GetCurrentName();
		FDataTableEditorUtils::RenameRow(DataTable.Get(), OldName, NewName);
		SelectRow(NewName);
	}
}

FReply SRowEditor::OnResetToDefaultClicked()
{
	if (DataTable.IsValid() && SelectedName.IsValid())
	{
		FDataTableEditorUtils::ResetToDefault(DataTable.Get(), *SelectedName.Get());
	}

	return FReply::Handled();
}

EVisibility SRowEditor::GetResetToDefaultVisibility() const
{
	EVisibility VisibleState = EVisibility::Collapsed;

	if (DataTable.IsValid() && SelectedName.IsValid())
	{
		if (FDataTableEditorUtils::DiffersFromDefault(DataTable.Get(), *SelectedName.Get()))
		{
			VisibleState = EVisibility::Visible;
		}
	}

	return VisibleState;
}

void SRowEditor::Construct(const FArguments& InArgs, UDataTable* Changed)
{
	ConstructInternal(Changed);
}

void SRowEditor::ConstructInternal(UDataTable* Changed)
{
	DataTable = Changed;
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs ViewArgs;
		ViewArgs.bAllowSearch = true;
		ViewArgs.bHideSelectionTip = false;
		ViewArgs.bShowOptions = false;
		ViewArgs.bShowObjectLabel = false;
		ViewArgs.NotifyHook = this;

		FStructureDetailsViewArgs StructureViewArgs;
		StructureViewArgs.bShowObjects = false;
		StructureViewArgs.bShowAssets = true;
		StructureViewArgs.bShowClasses = true;
		StructureViewArgs.bShowInterfaces = false;

		StructureDetailsView = PropertyModule.CreateStructureDetailView(ViewArgs, StructureViewArgs, CurrentRow, LOCTEXT("RowValue", "Row Value"));
	}

	RefreshNameList();
	Restore();
	const float ButtonWidth = 85.0f;
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2)
			[
				SNew(SBox)
				.WidthOverride(2 * ButtonWidth)
				.ToolTipText(LOCTEXT("SelectedRowTooltip", "Select a row to edit"))
				[
					SAssignNew(RowComboBox, SComboBox<TSharedPtr<FName>>)
					.OptionsSource(&CachedRowNames)
					.OnSelectionChanged(this, &SRowEditor::OnSelectionChanged)
					.OnGenerateWidget(this, &SRowEditor::OnGenerateWidget)
					.Content()
					[
						SNew(STextBlock).Text(this, &SRowEditor::GetCurrentNameAsText)
					]
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2)
			[
				SNew(SButton)
				.OnClicked(this, &SRowEditor::OnResetToDefaultClicked)
				.Visibility(this, &SRowEditor::GetResetToDefaultVisibility)
				.ContentPadding(FMargin(5.f, 0.f))
				.ToolTipText(LOCTEXT("ResetToDefaultToolTip", "Reset to Default"))
				.ButtonStyle(FAppStyle::Get(), "NoBorder")
				.ForegroundColor(FSlateColor::UseForeground())
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
				]
			]

		]
		+ SVerticalBox::Slot()
		[
			StructureDetailsView->GetWidget().ToSharedRef()
		]
	];
}

bool SRowEditor::IsMoveRowUpEnabled() const
{
	return true;
}

bool SRowEditor::IsMoveRowDownEnabled() const
{
	return true;
}

bool SRowEditor::IsAddRowEnabled() const
{
	return true;
}

bool SRowEditor::IsRemoveRowEnabled() const
{
	return true;
}

EVisibility SRowEditor::GetRenameVisibility() const
{
	return EVisibility::Visible;
}

#undef LOCTEXT_NAMESPACE
