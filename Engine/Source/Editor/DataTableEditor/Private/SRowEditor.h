// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "DataTableEditorUtils.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Kismet2/StructureEditorUtils.h"
#include "Layout/Visibility.h"
#include "Misc/NotifyHook.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/SoftObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"

class FProperty;
class FStructOnScope;
class SWidget;
class UDataTable;
class UScriptStruct;
struct FPropertyChangedEvent;

DECLARE_DELEGATE_OneParam(FOnRowModified, FName /*Row name*/);
DECLARE_DELEGATE_OneParam(FOnRowSelected, FName /*Row name*/);

class SRowEditor : public SCompoundWidget
	, public FNotifyHook
	, public FStructureEditorUtils::INotifyOnStructChanged
	, public FDataTableEditorUtils::INotifyOnDataTableChanged
{
public:
	SLATE_BEGIN_ARGS(SRowEditor) {}
	SLATE_END_ARGS()

	SRowEditor();
	virtual ~SRowEditor();

	// FNotifyHook
	virtual void NotifyPreChange( FProperty* PropertyAboutToChange ) override;
	virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged ) override;

	// INotifyOnStructChanged
	virtual void PreChange(const class UUserDefinedStruct* Struct, FStructureEditorUtils::EStructureEditorChangeInfo Info) override;
	virtual void PostChange(const class UUserDefinedStruct* Struct, FStructureEditorUtils::EStructureEditorChangeInfo Info) override;

	// INotifyOnDataTableChanged
	virtual void PreChange(const UDataTable* Changed, FDataTableEditorUtils::EDataTableChangeInfo Info) override;
	virtual void PostChange(const UDataTable* Changed, FDataTableEditorUtils::EDataTableChangeInfo Info) override;

	FOnRowSelected RowSelectedCallback;

protected:

	TArray<TSharedPtr<FName>> CachedRowNames;
	TSharedPtr<FStructOnScope> CurrentRow;
	TSoftObjectPtr<UDataTable> DataTable; // weak obj ptr couldn't handle reimporting
	TSharedPtr<class IStructureDetailsView> StructureDetailsView;
	TSharedPtr<FName> SelectedName;
	TSharedPtr<SComboBox<TSharedPtr<FName>>> RowComboBox;

	void RefreshNameList();
	void CleanBeforeChange();
	void Restore();

	/** Functions for enabling, disabling, and hiding portions of the row editor */
	virtual bool IsMoveRowUpEnabled() const;
	virtual bool IsMoveRowDownEnabled() const;
	virtual bool IsAddRowEnabled() const;
	virtual bool IsRemoveRowEnabled() const;
	virtual EVisibility GetRenameVisibility() const;

	UScriptStruct* GetScriptStruct() const;

	FName GetCurrentName() const;
	FText GetCurrentNameAsText() const;
	FString GetStructureDisplayName() const;
	TSharedRef<SWidget> OnGenerateWidget(TSharedPtr<FName> InItem);
	virtual void OnSelectionChanged(TSharedPtr<FName> InItem, ESelectInfo::Type InSeletionInfo);

	virtual FReply OnAddClicked();
	virtual FReply OnRemoveClicked();
	virtual FReply OnMoveRowClicked(FDataTableEditorUtils::ERowMoveDirection MoveDirection);
	FReply OnMoveToExtentClicked(FDataTableEditorUtils::ERowMoveDirection MoveDirection);
	void OnRowRenamed(const FText& Text, ETextCommit::Type CommitType);
	FReply OnResetToDefaultClicked();
	EVisibility GetResetToDefaultVisibility() const ;

	void ConstructInternal(UDataTable* Changed);

public:

	void Construct(const FArguments& InArgs, UDataTable* Changed);

	void SelectRow(FName Name);

	void HandleUndoRedo();

};
