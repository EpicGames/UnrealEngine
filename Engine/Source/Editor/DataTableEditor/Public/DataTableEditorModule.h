// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/IToolkit.h"
#include "Toolkits/IToolkitHost.h"

class FName;
class IDataTableEditor;
class UDataTable;

/** DataTable Editor module */
class FDataTableEditorModule : public IModuleInterface,
	public IHasMenuExtensibility, public IHasToolBarExtensibility
{

public:
	// IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/**
	 * Creates an instance of table editor object.  Only virtual so that it can be called across the DLL boundary.
	 *
	 * @param	Mode					Mode that this editor should operate in
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	Table					The table to start editing
	 *
	 * @return	Interface to the new table editor
	 */
	virtual TSharedRef<IDataTableEditor> CreateDataTableEditor( const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UDataTable* Table );

	/** Gets the extensibility managers for outside entities to extend static mesh editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override {return MenuExtensibilityManager;}

	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }
	
	/** DataTable Editor app identifier string */
	static const FName DataTableEditorAppIdentifier;

private:
	TSharedRef<IDataTableEditor> CreateStandardDataTableEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UDataTable* Table);

	TSharedRef<IDataTableEditor> CreateCompositeDataTableEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UDataTable* Table);

	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
;

};


