// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataTableEditorUtils.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "DataTableRowMenuContext.generated.h"

class FDataTableEditor;

UCLASS()
class DATATABLEEDITOR_API UDataTableRowMenuContext : public UAssetEditorToolkitMenuContext
{
	GENERATED_BODY()
	
public:

	FDataTableEditorRowListViewDataPtr RowDataPtr;
};
